{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.2",
	"FriendlyName": "Recovery Hub",
	"Description": "Track changes in the Editor to allow recovery in the event of a crash",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "DisasterRecoveryClient",
			"Type": "EditorNoCommandlet",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ConcertSyncClient",
			"Enabled": true
		},
        {
          "Name": "ConcertSharedSlate",
          "Enabled": true
        }
	]
}
