// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DisasterRecoveryClient : ModuleRules
	{
		public DisasterRecoveryClient(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Concert",
					"ConcertClient",
					"ConcertSyncClient",
					"ConcertSyncCore",
					"ConcertSharedSlate",
					"ConcertTransport",
					"DesktopPlatform",
					"Json",
					"EditorFramework",
					"UnrealEd",
					"EditorStyle",
					"InputCore",
					"Serialization",
					"Slate",
					"SlateCore",
					"ToolWidgets",
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"WorkspaceMenuStructure",
						"DirectoryWatcher",
					}
				);
			}
		}
	}
}
