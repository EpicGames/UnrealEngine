// Copyright Epic Games, Inc. All Rights Reserved.

#include "IDisasterRecoveryClientModule.h"

#include "ConcertSyncSessionFlags.h"
#include "Framework/Application/SlateApplication.h"
#include "IConcertSyncClientModule.h"
#include "IConcertClient.h"
#include "IConcertClientWorkspace.h"
#include "IConcertSyncClient.h"
#include "ConcertClientSettings.h"
#include "ConcertFrontendStyle.h"
#include "ConcertLocalFileSharingService.h"

#include "Misc/Paths.h"
#include "Misc/CoreDelegates.h"

#include "DisasterRecoverySettings.h"
#include "DisasterRecoverySessionManager.h"
#include "DisasterRecoveryUtil.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "IPackageAutoSaver.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"

#if WITH_EDITOR
	#include "ISettingsModule.h"
	#include "ISettingsSection.h"
	#include "WorkspaceMenuStructure.h"
	#include "WorkspaceMenuStructureModule.h"
	#include "Widgets/Docking/SDockTab.h"
	#include "SDisasterRecoveryHub.h"
#endif

LLM_DEFINE_TAG(Concert_DisasterRecoveryClient);
DEFINE_LOG_CATEGORY(LogDisasterRecovery);

#define LOCTEXT_NAMESPACE "DisasterRecoveryClient"

namespace DisasterRecoveryUtil
{
static const FName RecoveryHubTabName("RecoveryHub");

bool IsRecoveryServiceHostedInCrashReporter()
{
	//return FGenericCrashContext::IsOutOfProcessCrashReporter();
	return false; // In 4.25.1, this was disabled because CRC suspiciously crashes.
}

/**
 * Return the name of the executable hosting disaster recovery service, like 'UnrealRecoverySvc' without the extension.
 */
FString GetDisasterRecoveryServiceExeName()
{
	if (IsRecoveryServiceHostedInCrashReporter())
	{
		return TEXT("CrashReporterClientEditor");
	}
	else
	{
		return TEXT("UnrealRecoverySvc");
	}
}

}


/** Implement the Disaster Recovery module */
class FDisasterRecoveryClientModule : public IDisasterRecoveryClientModule
{
public:
	virtual void StartupModule() override
	{
		LLM_SCOPE_BYTAG(Concert_DisasterRecoveryClient);

		if (!FApp::HasProjectName())
		{
			UE_LOGF(LogDisasterRecovery, Warning, "The current project doesn't have any name. Recovery Hub will be disabled! Please set your project name.");
			return; // Needs a project name.
		}

		Role = TEXT("DisasterRecovery");

		// Hook to the PreExit callback, needed to execute UObject related shutdowns
		FCoreDelegates::OnPreExit.AddRaw(this, &FDisasterRecoveryClientModule::HandleAppPreExit);

		// Wait for init to finish before starting the Disaster Recovery service
		FCoreDelegates::OnFEngineLoopInitComplete.AddRaw(this, &FDisasterRecoveryClientModule::OnEngineInitComplete);

		// Initialize Style
		FConcertFrontendStyle::Initialize();

		const FString DisasterRecoveryServerName = RecoveryService::GetRecoveryServerName();
		const FString DisasterRecoverySessionName = RecoveryService::MakeSessionName();

		// Create and populate the client config object
		UConcertClientConfig* ClientConfig = NewObject<UConcertClientConfig>();
		ClientConfig->bIsHeadless = true;
		ClientConfig->bInstallEditorToolbarButton = false;
		ClientConfig->bAutoConnect = false;
		ClientConfig->DefaultServerURL = DisasterRecoveryServerName;
		ClientConfig->DefaultSessionName = DisasterRecoverySessionName;
		ClientConfig->DefaultSaveSessionAs = DisasterRecoverySessionName;
		//ClientConfig->ClientSettings.DiscoveryTimeoutSeconds = 0; // Setting this to zero prevents showing 'Server {serverName} lost." but will not detect when a server dies, growing the list of known server indefinitely.
		ClientConfig->EndpointSettings.RemoteEndpointTimeoutSeconds = 0; // Ensure the endpoints never time out (and are kept alive automatically by Concert).
		ClientConfig->ClientSettings.ClientAuthenticationKey = DisasterRecoveryServerName; // The server adds its own server name to the list of authorized client keys, use that key to authorize this client on the server.

		// Create and start the client.
		DisasterRecoveryClient = IConcertSyncClientModule::Get().CreateClient(Role);
		DisasterRecoveryClient->SetFileSharingService(MakeShared<FConcertLocalFileSharingService>(Role));
		DisasterRecoveryClient->Startup(ClientConfig, EConcertSyncSessionFlags::Default_DisasterRecoverySession);

		// Hook to sync session is created/destroyed event to detect when a Multi-User session (incompatible) is started.
		IConcertSyncClientModule::Get().OnClientCreated().AddRaw(this, &FDisasterRecoveryClientModule::HandleConcertSyncClientCreated);
		for (TSharedRef<IConcertSyncClient> Client : IConcertSyncClientModule::Get().GetClients())
		{
			Client->OnSyncSessionStartup().AddRaw(this, &FDisasterRecoveryClientModule::HandleSyncSessionStartup);
			Client->OnSyncSessionShutdown().AddRaw(this, &FDisasterRecoveryClientModule::HandleSyncSessionShutdown);
		}

		// Register the Disaster Recovery Settings panel.
		RegisterSettings();
	}

	virtual void ShutdownModule() override
	{
		LLM_SCOPE_BYTAG(Concert_DisasterRecoveryClient);

		FCoreDelegates::OnFEngineLoopInitComplete.RemoveAll(this);

		// Unhook AppPreExit and call it
		FCoreDelegates::OnPreExit.RemoveAll(this);
		HandleAppPreExit();

		// Unhook this module callback from other clients.
		if (IConcertSyncClientModule::IsAvailable())
		{
			IConcertSyncClientModule::Get().OnClientCreated().RemoveAll(this);
			for (TSharedRef<IConcertSyncClient> Client : IConcertSyncClientModule::Get().GetClients())
			{
				Client->OnSyncSessionStartup().RemoveAll(this);
				Client->OnSyncSessionShutdown().RemoveAll(this);
			}
		}

#if WITH_EDITOR
		// Unregister the recovery tab spawner.
		UnregisterTabSpawner();
#endif

		// Unregister the Disaster Recovery Settings panel.
		UnregisterSettings();
	}

	virtual TSharedPtr<IConcertSyncClient> GetClient() const override
	{
		return DisasterRecoveryClient;
	}

private:
	void RegisterSettings()
	{
		if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			ISettingsSectionPtr SettingsSection = SettingsModule->RegisterSettings(DisasterRecoveryUtil::GetSettingsContainerName(), DisasterRecoveryUtil::GetSettingsCategoryName(), DisasterRecoveryUtil::GetSettingsSectionName(),
				LOCTEXT("DisasterRecoverySettingsName", "Recovery Hub"),
				LOCTEXT("DisasterRecoverySettingsDescription", "Configure the Recovery Hub Settings."),
				GetMutableDefault<UDisasterRecoverClientConfig>());

			if (SettingsSection.IsValid())
			{
				SettingsSection->OnModified().BindRaw(this, &FDisasterRecoveryClientModule::HandleSettingsSaved);
			}
		}
	}

	void UnregisterSettings()
	{
		if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->UnregisterSettings(DisasterRecoveryUtil::GetSettingsContainerName(), DisasterRecoveryUtil::GetSettingsCategoryName(), DisasterRecoveryUtil::GetSettingsSectionName());
		}
	}

	bool HandleSettingsSaved()
	{
		check (DisasterRecoveryClient);

		const UDisasterRecoverClientConfig* Config = GetDefault<UDisasterRecoverClientConfig>();
		if (SessionManager && !Config->bIsEnabled)
		{
			SessionManager->LeaveSession(); // Stop the current session (if any).
		}
		else if (Config->bIsEnabled && (!SessionManager || !SessionManager->HasInProgressSession()))
		{
			StartDisasterRecoveryService(); // Restart (create a new session).
		}

		return true;
	}

#if WITH_EDITOR
	void RegisterTabSpawner()
	{
		TSharedPtr<FWorkspaceItem> WorkspaceGroup = WorkspaceMenu::GetMenuStructure().GetToolsCategory();
		if (bTabSpawnerRegistrated)
		{
			UnregisterTabSpawner();
		}

		bTabSpawnerRegistrated = true;

		FTabSpawnerEntry& BrowserSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(DisasterRecoveryUtil::RecoveryHubTabName,
			FOnSpawnTab::CreateRaw(this, &FDisasterRecoveryClientModule::SpawnRecoveryHub))
			.SetIcon(FSlateIcon(FConcertFrontendStyle::GetStyleSetName(), TEXT("Concert.RecoveryHub")))
			.SetDisplayName(LOCTEXT("RecoveryHubTabTitle", "Recovery Hub"))
			.SetTooltipText(LOCTEXT("RecoveryHubTabTooltip", "Open the Recovery Hub"))
			.SetMenuType(ETabSpawnerMenuType::Enabled);

		if (WorkspaceGroup.IsValid())
		{
			BrowserSpawnerEntry.SetGroup(WorkspaceGroup.ToSharedRef());
		}
	}

	void UnregisterTabSpawner()
	{
		bTabSpawnerRegistrated = false;
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(DisasterRecoveryUtil::RecoveryHubTabName);
	}

	TSharedRef<SDockTab> SpawnRecoveryHub(const FSpawnTabArgs& SpawnTabArgs)
	{
		const TSharedRef<SDockTab> DockTab = SNew(SDockTab).TabRole(ETabRole::NomadTab);
		TSharedRef<SDisasterRecoveryHub> RecoveryHub = SNew(SDisasterRecoveryHub, DockTab, SpawnTabArgs.GetOwnerWindow(), SessionManager);
		DockTab->SetContent(RecoveryHub);
		return DockTab;
	}
#endif

	void OnEngineInitComplete()
	{
		StartDisasterRecoveryService();
	}

	// Module shutdown is dependent on the UObject system which is currently shutdown on AppExit
	void HandleAppPreExit()
	{
		// if UObject system isn't initialized, skip shutdown
		if (!UObjectInitialized())
		{
			return;
		}

		// Shutdown the manager, disconnecting from the live session normally (if any).
		SessionManager.Reset();

		if (DisasterRecoveryClient)
		{
			DisasterRecoveryClient->Shutdown();
			DisasterRecoveryClient.Reset();
		}
	}

	void HandleConcertSyncClientCreated(TSharedRef<IConcertSyncClient> Client)
	{
		if (Client->GetConcertClient()->GetRole() != Role) // Exclude disaster recovery own session connection changes.
		{
			Client->OnSyncSessionStartup().AddRaw(this, &FDisasterRecoveryClientModule::HandleSyncSessionStartup);
			Client->OnSyncSessionShutdown().AddRaw(this, &FDisasterRecoveryClientModule::HandleSyncSessionShutdown);
		}
	}

	void HandleSyncSessionStartup(const IConcertSyncClient* SyncClient)
	{
		if (DisasterRecoveryClient.Get() != SyncClient)
		{
			SetIgnoreOnRestoreState(!IsCompatibleWithOtherConcertSessions(SyncClient, /*SyncClientShuttingDownSession*/nullptr));
		}
	}

	void HandleSyncSessionShutdown(const IConcertSyncClient* SyncClient)
	{
		if (DisasterRecoveryClient.Get() != SyncClient)
		{
			SetIgnoreOnRestoreState(!IsCompatibleWithOtherConcertSessions(/*SyncClientStartingSession*/nullptr, SyncClient));
		}
	}

	static FString GetDisasterRecoveryServicePath()
	{
		auto GetDisasterRecoveryServicePathForBuildConfiguration = [](const EBuildConfiguration InBuildConfiguration) -> FString
		{
			FString ServicePath = FPlatformProcess::GenerateApplicationPath(DisasterRecoveryUtil::GetDisasterRecoveryServiceExeName(), InBuildConfiguration);
			return FPaths::FileExists(ServicePath) ? ServicePath : FString();
		};

		// First try and use our build configuration
		FString ServicePath = GetDisasterRecoveryServicePathForBuildConfiguration(FApp::GetBuildConfiguration());

		// Fall back to Development if the app doesn't exist for our build configuration, as installed builds only build it for Development
		if (ServicePath.IsEmpty() && FApp::GetBuildConfiguration() != EBuildConfiguration::Development)
		{
			ServicePath = GetDisasterRecoveryServicePathForBuildConfiguration(EBuildConfiguration::Development);
		}

		return ServicePath;
	}

	/** Spawn the disaster recovery service. To use when the 'out of process crash reporter' is not supported on the platform (Linux and Mac). */
	bool SpawnDisasterRecoveryServer(const FString& ServerName)
	{
		check(!DisasterRecoveryUtil::IsRecoveryServiceHostedInCrashReporter()); // Should use the out-of-process crash reporter.

		if (DisasterRecoveryServiceHandle.IsValid())
		{
			return true; // Already running.
		}

		// Find the service path that will host the sync server
		const FString DisasterRecoveryServicePath = GetDisasterRecoveryServicePath();
		if (DisasterRecoveryServicePath.IsEmpty())
		{
			UE_LOGF(LogDisasterRecovery, Warning, "Unreal Recovery Service application was not found. Recovery Hub will be disabled! Please build '%ls'.", *DisasterRecoveryUtil::GetDisasterRecoveryServiceExeName());
			return false;
		}

		FString DisasterRecoveryServiceCommandLine;
		DisasterRecoveryServiceCommandLine += FString::Printf(TEXT(" -ConcertServer=\"%s\""), *ServerName);
		DisasterRecoveryServiceCommandLine += FString::Printf(TEXT(" -EditorPID=%d"), FPlatformProcess::GetCurrentProcessId());

		// Create the service process that will host the sync server
		DisasterRecoveryServiceHandle = FPlatformProcess::CreateProc(*DisasterRecoveryServicePath, *DisasterRecoveryServiceCommandLine, true, true, true, nullptr, 0, nullptr, nullptr, nullptr);
		if (!DisasterRecoveryServiceHandle.IsValid())
		{
			UE_LOGF(LogDisasterRecovery, Error, "Failed to launch Recovery Service application. Recovery Hub will be disabled!");
			return false;
		}

		return true;
	}

	/** Starts or restarts the recovery service. */
	void StartDisasterRecoveryService()
	{
		LLM_SCOPE_BYTAG(Concert_DisasterRecoveryClient);

		// Started for the first time (not restarted)? Always create the manager (and start the service) even if disaster recovery is disabled (in the settings). This allows the user to import sessions for crash analysis.
		if (!SessionManager)
		{
			// If crash reporter is running out of process, it also hosts disaster recovery server as the '-ConcertServer' param is set when spawning CrashReporterClientEditor. No need to start the UnrealRecoverySvc executable.
			if (!DisasterRecoveryUtil::IsRecoveryServiceHostedInCrashReporter() && !SpawnDisasterRecoveryServer(RecoveryService::GetRecoveryServerName()))
			{
				return; // Failed to spawn the service.
			}

			// Create the session manager.
			SessionManager = MakeShared<FDisasterRecoverySessionManager>(Role, DisasterRecoveryClient);

#if WITH_EDITOR
			// Register the tab spawner (used to spawn the recovery hub).
			RegisterTabSpawner();
#endif

			// Set all events captured by the disaster recovery service as 'restorable' unless another concert client (assumed Multi-User) has created an incompatible session.
			SetIgnoreOnRestoreState(!IsCompatibleWithOtherConcertSessions(/*SyncClientStartingSession*/nullptr, /*SyncClientShuttingDownSession*/nullptr));

			// Prevent creating/restoring sessions if disaster recovery is disabled.
			if (!GetDefault<UDisasterRecoverClientConfig>()->bIsEnabled)
			{
				return;
			}

			// Prevent the "Auto-Save" system asking the user to restore auto-saved packages. Disaster recovery session already contains the auto-saved packages.
			if (GUnrealEd)
			{
				GUnrealEd->GetPackageAutoSaver().DisableRestorePromptAndDeclinePackageRecovery();
			}
		}

		// In unattended mode, create a new session. This will move 'active/crashed' session(s) in the 'recent' list where the user will be able to restore them later if needed.
		if (FApp::IsUnattended())
		{
			UE_LOGF(LogDisasterRecovery, Display, "Skipped recovery process. The application was launched in unattended mode. Previous sessions will be available from recovery hub 'recent' list.");
			SessionManager->CreateAndJoinSession(); // Note: Concert already provides toast notifications in case of success or failure.
			return;
		}

		// Used as token to prevent future continuation to execution if 'this' get deleted in the mean time.
		TWeakPtr<FDisasterRecoverySessionManager> WeakSessionManager = SessionManager;

		// Check if a session is candidate for restoration.
		SessionManager->HasRecoverableCandidates().Next([WeakSessionManager](bool CandidateFound)
		{
			TSharedPtr<FDisasterRecoverySessionManager> SessionManagerPin = WeakSessionManager.Pin();
			if (!SessionManagerPin || !GetDefault<UDisasterRecoverClientConfig>()->bIsEnabled)
			{
				return; // The FDisasterRecoveryClientModule instance was deleted or the recovery disabled.
			}

			if (!CandidateFound)
			{
				SessionManagerPin->CreateAndJoinSession();
				return;
			}

			// Launch the recovery hub UI.
			TSharedRef<SWindow> NewWindow = SNew(SWindow)
				.Title(LOCTEXT("RecoveryTitle", "Recovery Hub"))
				.SizingRule(ESizingRule::UserSized)
				.ClientSize(FVector2D(1200, 800))
				.IsTopmostWindow(true) // Keep it on top. Cannot be modal at the moment because Concert doesn't tick during a modal dialog.
				.SupportsMaximize(true)
				.SupportsMinimize(false);

			TSharedRef<SDisasterRecoveryHub> RecoveryHub =
				SNew(SDisasterRecoveryHub, nullptr, NewWindow, SessionManagerPin)
				.IntroductionText(LOCTEXT("RecoveryIntroductionText", "An abnormal Editor termination was detected for this project. You can recover up to the last operation recorded or to a previous state."))
				.IsRecoveryMode(true);

			NewWindow->SetContent(RecoveryHub);
			NewWindow->SetOnWindowClosed(FOnWindowClosed::CreateLambda([WeakSessionManager](const TSharedRef<SWindow>&)
			{
				if (TSharedPtr<FDisasterRecoverySessionManager> SessionManagerPin = WeakSessionManager.Pin())
				{
					if (!SessionManagerPin->HasInProgressSession() && GetDefault<UDisasterRecoverClientConfig>()->bIsEnabled) // The user clicked 'Cancel' or closed the window without restoring anything.
					{
						SessionManagerPin->CreateAndJoinSession();
					}
				}
			}));

			FSlateApplication::Get().AddWindow(NewWindow, true);
		});
	}

	/** Returns true if disaster recovery Concert session can run concurrently with other Concert sessions (if any). */
	bool IsCompatibleWithOtherConcertSessions(const IConcertSyncClient* SyncClientStartingSession, const IConcertSyncClient* SyncClientShuttingDownSession) const
	{
		// At the moment, we don't expect more than 2 clients. We don't have use cases for a third concurrent concert client.
		checkf(IConcertSyncClientModule::Get().GetClients().Num() <= 2, TEXT("Expected 1 recovery client + 1 multi-user client at max."));

		// Scan all existing clients.
		for (const TSharedRef<IConcertSyncClient>& SyncClient : IConcertSyncClientModule::Get().GetClients())
		{
			if (SyncClient == DisasterRecoveryClient || &SyncClient.Get() == SyncClientShuttingDownSession)
			{
				continue; // Compatible with itself or the sync client is shutting down its sync session, so it cannot interfere anymore.
			}
			else if (&SyncClient.Get() == SyncClientStartingSession)
			{
				if (!IsCompatibleWithConcertClient(&SyncClient.Get()))
				{
					return false; // The sync client starting a session will interfere with disaster recovery client.
				}
			}
			else if (SyncClient->GetWorkspace() && !IsCompatibleWithConcertClient(&SyncClient.Get())) // A valid workspace means the client is joining, in or leaving a session.
			{
				return false; // That existing client is interfering with disaster recovery client.
			}
		}

		return true; // No other sessions exist or it is compatible.
	}

	bool IsCompatibleWithConcertClient(const IConcertSyncClient* SyncClient) const
	{
		check(SyncClient != DisasterRecoveryClient.Get());
		checkf(SyncClient->GetConcertClient()->GetRole() == TEXT("MultiUser"), TEXT("A new Concert role was added, check if this role can run concurrently with recovery service."));

		// Multi-User (MU) sessions are not compatible with disaster recovery (DR) session because MU events are performed in a transient sandbox that doesn't exist outside the MU session.
		// If a crash occurs during a MU session, DR must not recover transactions applied to the transient sandbox. DR will will record the MU events, but for crash inspection purpose only.
		return SyncClient->GetConcertClient()->GetRole() != TEXT("MultiUser");
	}

	/** Sets whether further Concert events (transaction/package) emitted by Disaster Recovery have the 'ignore' flag on or off. */
	void SetIgnoreOnRestoreState(bool bIgnore)
	{
		if (TSharedPtr<IConcertClientWorkspace> Workspace = DisasterRecoveryClient ? DisasterRecoveryClient->GetWorkspace() : TSharedPtr<IConcertClientWorkspace>())
		{
			Workspace->SetIgnoreOnRestoreFlagForEmittedActivities(bIgnore);
		}
	}

private:
	/** This client role, a tag given to different types of concert client, i.e. DisasterRecovery for this one. */
	FString Role;

	/** Sync client handling disaster recovery */
	TSharedPtr<IConcertSyncClient> DisasterRecoveryClient;

	/** Handle to the active disaster recovery service app, if any */
	FProcHandle DisasterRecoveryServiceHandle;

	/** The disaster recovery session manager for this instance. */
	TSharedPtr<FDisasterRecoverySessionManager> SessionManager;

	/** Whether the recovery browser tab spawner was registered or not.*/
	bool bTabSpawnerRegistrated = false;
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FDisasterRecoveryClientModule, DisasterRecoveryClient);

