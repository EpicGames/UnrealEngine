// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisasterRecoverySessionInfo.generated.h"

/** Flags describing the type of recovery sessions. */
enum class EDisasterRecoverySessionFlags : uint8
{
	/** No flags. */
	None = 0,

	/** The session was terminated abnormally. */
	AbnormalTerminaison = 1 << 0,

	/** The session was imported. */
	Imported = 1 << 1,

	/** The session was processed and moved to the recent list. */
	Recent = 1 << 2,

	/** Indicate if the debugger was attached when the session was created. */
	DebuggerAttached = 1 << 3,
};
ENUM_CLASS_FLAGS(EDisasterRecoverySessionFlags)

/** Information about a single session info. */
USTRUCT()
struct FDisasterRecoverySession
{
	GENERATED_BODY()

	/** The repository ID created on the server to store that session. */
	UPROPERTY()
	FGuid RepositoryId;

	/** The immediate parent directory containing this session repository. */
	UPROPERTY()
	FString RepositoryRootDir;

	/** The name of the session. */
	UPROPERTY()
	FString SessionName;

	/** The client that mounted that session repository. */
	UPROPERTY()
	uint32 MountedByProcessId = 0;

	/** The process ID of the client connected to the session (the session is live and recording transactions for this client process). */
	UPROPERTY()
	uint32 ClientProcessId = 0;

	/** Information about the session. */
	UPROPERTY()
	uint8 Flags = static_cast<uint8>(EDisasterRecoverySessionFlags::None);

	/** Returns true if the session is currently in-progress (i.e. the session is live and has a client) */
	bool IsLive() const { return ClientProcessId != 0; }

	/** Returns true if the session was moved to the recent list. */
	bool IsRecent() const { return EnumHasAnyFlags(static_cast<EDisasterRecoverySessionFlags>(Flags), EDisasterRecoverySessionFlags::Recent); }

	/** Returns true if the session was imported for inspection. It can be from any project and likely not recoverable. */
	bool IsImported() const { return EnumHasAnyFlags(static_cast<EDisasterRecoverySessionFlags>(Flags), EDisasterRecoverySessionFlags::Imported); }

	/** Returns true if the debugger was attached to this session. */
	bool WasDebuggerAttached() const { return EnumHasAnyFlags(static_cast<EDisasterRecoverySessionFlags>(Flags), EDisasterRecoverySessionFlags::DebuggerAttached); }

	/** Session was abnormally terminated */
	bool WasAbnormallyTerminated() const { return EnumHasAnyFlags(static_cast<EDisasterRecoverySessionFlags>(Flags), EDisasterRecoverySessionFlags::AbnormalTerminaison); }

	/** Returns true if the session abnormally terminated and the user did not have the change to inspect/recover it. */
	bool IsUnreviewedCrash() const { return WasAbnormallyTerminated() && !IsLive() && !IsRecent() && !IsImported(); }

	/** Returns true if this session repository is mounted (by any process). */
	bool IsMounted() const { return MountedByProcessId != 0; }

	/** Compares if two recovery sessions are equals. */
	bool operator==(const FDisasterRecoverySession& Other) const
	{
		return RepositoryId == Other.RepositoryId &&
			RepositoryRootDir == Other.RepositoryRootDir &&
			SessionName == Other.SessionName &&
			MountedByProcessId == Other.MountedByProcessId &&
			ClientProcessId == Other.ClientProcessId &&
			Flags == Other.Flags;
	}

	/** Compares if two recovery sessions are different. */
	bool operator!=(const FDisasterRecoverySession& Other) const
	{
		return !operator==(Other);
	}
};

/** Information about a disaster recovery client. */
USTRUCT()
struct FDisasterRecoveryClientInfo
{
	GENERATED_BODY()

	/** The client process ID. */
	UPROPERTY()
	uint32 ClientProcessId = 0;

	/** The client app ID. */
	UPROPERTY()
	FGuid ClientAppId;
};


/**
 * Hold the information for multiple disaster recovery sessions.
 */
USTRUCT()
struct FDisasterRecoveryInfo
{
	GENERATED_BODY()

	/** The revision number of the information. Updated everytime the recovery info file is written. */
	UPROPERTY()
	uint32 Revision = 0;

	/** The list of running/crashing/crashed sessions. */
	UPROPERTY()
	TArray<FDisasterRecoverySession> ActiveSessions;

	/** The list of recent sessions (rotated over time). */
	UPROPERTY()
	TArray<FDisasterRecoverySession> RecentSessions;

	/** The list of imported sessions (rotated over time). */
	UPROPERTY()
	TArray<FDisasterRecoverySession> ImportedSessions;

	/** The list of session being created, for which the repository was created and mounted, but the session not yet started. */
	UPROPERTY()
	TArray<FDisasterRecoverySession> PendingSessions;

	/** The list of session reporitories ID that are scheduled to be discarded, but kept around until the server confirms the deletion. */
	UPROPERTY()
	TArray<FGuid> DiscardedRepositoryIds;

	/** The list of client currently executing (in different processes). */
	UPROPERTY()
	TArray<FDisasterRecoveryClientInfo> Clients;
};
