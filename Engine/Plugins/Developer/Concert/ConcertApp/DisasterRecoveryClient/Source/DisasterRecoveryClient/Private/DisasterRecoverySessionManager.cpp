// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisasterRecoverySessionManager.h"
#include "ConcertSyncSessionTypes.h"
#include "IConcertClient.h"
#include "DisasterRecoverySessionInfo.h"
#include "IDisasterRecoveryClientModule.h"
#include "ConcertActivityStream.h"
#include "ConcertClientSettings.h"
#include "DisasterRecoverySettings.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"

#include "Backends/JsonStructSerializerBackend.h"
#include "Backends/JsonStructDeserializerBackend.h"
#include "HAL/FileManager.h"
#include "IConcertSyncClient.h"
#include "Serialization/JsonSerializer.h"
#include "Misc/TVariantMeta.h"
#include "StructSerializer.h"
#include "StructDeserializer.h"

#if WITH_EDITOR
#include "IDirectoryWatcher.h"
#include "DirectoryWatcherModule.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "DisasterRecoverySessionManager"

namespace DisasterRecoveryUtil
{

/** Return the text messages displayed when an operation is aborted. */
FText GetOperationAbortedMessage()
{
	return LOCTEXT("AbortedMessage", "The operation was aborted.");
}

/** Checks if two running process IDs are instances of the same executable. */
bool IsSameExecutable(int32 LhsProcessId, int32 RhsProcessId)
{
	return FPaths::GetPathLeaf(FPlatformProcess::GetApplicationName(LhsProcessId)) == FPaths::GetPathLeaf(FPlatformProcess::GetApplicationName(RhsProcessId));
};

/** Check whether a given recovery client process is running */
bool IsRecoveryClientProcessRunning(int32 Pid)
{
	return FPlatformProcess::IsApplicationRunning(Pid) && IsSameExecutable(Pid, FPlatformProcess::GetCurrentProcessId());
}

/** Check whether the recovery session client process is running. */
bool IsSessionClientProcessRunning(const FDisasterRecoverySession& Session)
{
	return Session.ClientProcessId != 0 && IsRecoveryClientProcessRunning(Session.ClientProcessId);
};

/** Returns a system wide unique mutex name to access the shared file. */
const TCHAR* GetSystemMutexName()
{
	return TEXT("Unreal_DisasterRecovery_4221FF"); // This is an arbitrary name. Need to be unique enough to avoid clashing with other applications.
}

/** Load the disaster recovery info file in memory. */
bool LoadDisasterRecoveryInfo(const FString& Pathname, FDisasterRecoveryInfo& OutInfo)
{
	if (TUniquePtr<FArchive> FileReader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*Pathname)))
	{
		FJsonStructDeserializerBackend Backend(*FileReader);
		FStructDeserializer::Deserialize(OutInfo, Backend);

		FileReader->Close();
		return !FileReader->IsError();
	}

	return false;
}

/** Save the disaster recovery info file to disk. */
bool SaveDisasterRecoveryInfo(const FString& Pathname, const FDisasterRecoveryInfo& InInfo)
{
	if (TUniquePtr<FArchive> FileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*Pathname)))
	{
		FJsonStructSerializerBackend Backend(*FileWriter, EStructSerializerBackendFlags::Default);
		FStructSerializer::Serialize(InInfo, Backend);

		FileWriter->Close();
		return !FileWriter->IsError();
	}

	return false;
}

/** Load the recovery info file on creation and auto-save it when exiting the scope unless auto-save is disabled. */
class ScopedRecoveryInfoExclusiveUpdater
{
public:
	ScopedRecoveryInfoExclusiveUpdater(const FString& InPathname, uint32& OutWrittenRevision)
		: Pathname(InPathname)
		, WrittenRevision(OutWrittenRevision)
		, SystemWideMutex(GetSystemMutexName()) // Get machine-wide exclusive access to the file during the scope.
	{
		LoadDisasterRecoveryInfo(Pathname, RecoveryInfo);
	}

	~ScopedRecoveryInfoExclusiveUpdater()
	{
		if (bAutoSave)
		{
			RecoveryInfo.Revision += 1; // The file is going to be written, increase the revision number.
			SaveDisasterRecoveryInfo(Pathname, RecoveryInfo);
			WrittenRevision = RecoveryInfo.Revision;
		}
	}

	FDisasterRecoveryInfo& Info() { return RecoveryInfo; }
	void SetAutoSave(bool InAutoSave) { bAutoSave = InAutoSave; }

private:
	const FString& Pathname;
	uint32& WrittenRevision;
	FSystemWideCriticalSection SystemWideMutex;
	FDisasterRecoveryInfo RecoveryInfo;
	bool bAutoSave = true; // By default, automatically save when going out of scope.
};

/** Load the recovery info file and read its current version. */
uint32 GetDisasterRecoveryInfoVersion(const FString& InPathname)
{
	// Get machine-wide exclusive access to the file.
	FSystemWideCriticalSection SystemWideMutex(DisasterRecoveryUtil::GetSystemMutexName());

	FDisasterRecoveryInfo RecoveryInfo;
	LoadDisasterRecoveryInfo(InPathname, RecoveryInfo);
	return RecoveryInfo.Revision;
}

/** Migrate the recovery info file from 4.24 to the new format in 4.25. */
void MigrateSessionInfoFrom_4_24(const FString& OldInfoPathname, FDisasterRecoveryInfo& OutRecoveryInfo)
{
	auto MigrateSessionsFn = [](const TArray<TSharedPtr<FJsonValue>>& SessionArray, FDisasterRecoveryInfo& OutRecoveryInfo, bool bHistoryList)
	{
		for (const TSharedPtr<FJsonValue>& SessionArrayElement : SessionArray)
		{
			const TSharedPtr<FJsonObject>* SessionObj = nullptr;
			if (SessionArrayElement->TryGetObject(SessionObj))
			{
				// All the expected field to extract from the session.
				FString RepositoryRootDir;
				FString LastSessionName;
				bool bAutoRestore = false;
				const TSharedPtr<FJsonObject>* RepositoryIdObj = nullptr;
				int32 A, B, C, D; // Guid components. Parsting them as uint32 directly doesn't work... need to read them as int32 and cast them to uint32.

				bool bAllRequiredFieldPresent = true;
				bAllRequiredFieldPresent &= (*SessionObj)->TryGetStringField(TEXT("RepositoryRootDir"), RepositoryRootDir);
				bAllRequiredFieldPresent &= (*SessionObj)->TryGetStringField(TEXT("LastSessionName"), LastSessionName);
				bAllRequiredFieldPresent &= (*SessionObj)->TryGetBoolField(TEXT("bAutoRestoreLastSession"), bAutoRestore);
				bAllRequiredFieldPresent &= (*SessionObj)->TryGetObjectField(TEXT("RepositoryId"), RepositoryIdObj);
				bAllRequiredFieldPresent &= (*RepositoryIdObj)->TryGetNumberField(TEXT("A"), A);
				bAllRequiredFieldPresent &= (*RepositoryIdObj)->TryGetNumberField(TEXT("B"), B);
				bAllRequiredFieldPresent &= (*RepositoryIdObj)->TryGetNumberField(TEXT("C"), C);
				bAllRequiredFieldPresent &= (*RepositoryIdObj)->TryGetNumberField(TEXT("D"), D);

				if (bAllRequiredFieldPresent)
				{
					FDisasterRecoverySession MigratedSession;
					MigratedSession.RepositoryId = FGuid((uint32)A, (uint32)B, uint32(C), uint32(D));
					MigratedSession.RepositoryRootDir = RepositoryRootDir;
					MigratedSession.SessionName = LastSessionName;

					if (bAutoRestore && !bHistoryList)
					{
						MigratedSession.Flags = static_cast<uint8>(EDisasterRecoverySessionFlags::AbnormalTerminaison);
						OutRecoveryInfo.ActiveSessions.Add(MoveTemp(MigratedSession));
					}
					else
					{
						MigratedSession.Flags = static_cast<uint8>(EDisasterRecoverySessionFlags::Recent);
						OutRecoveryInfo.RecentSessions.Add(MoveTemp(MigratedSession));
					}
				}
			}
		}
	};

	// Load the old .json file.
	TUniquePtr<FArchive> FileAr(IFileManager::Get().CreateFileReader(*OldInfoPathname));
	if (FileAr)
	{
		TSharedPtr<FJsonObject> RootObject = MakeShared<FJsonObject>();
		TSharedRef<TJsonReader<TCHAR>> Reader = TJsonReaderFactory<TCHAR>::Create(FileAr.Get());
		if (FJsonSerializer::Deserialize(Reader, RootObject))
		{
			// Migrate the very last sessions (possibly migrating crashes)
			const TArray<TSharedPtr<FJsonValue>>* SessionArray = nullptr;
			if (RootObject->TryGetArrayField(TEXT("Sessions"), SessionArray))
			{
				MigrateSessionsFn(*SessionArray, OutRecoveryInfo, /*bHistoryList*/false);
			}

			// Migrate the recent sessions.
			const TArray<TSharedPtr<FJsonValue>>* SessionHistoryArray = nullptr;
			if (RootObject->TryGetArrayField(TEXT("SessionHistory"), SessionHistoryArray))
			{
				MigrateSessionsFn(*SessionHistoryArray, OutRecoveryInfo, /*bHistoryList*/true);
			}
		}

		FileAr->Close();
	}
}

} // namespace DisasterRecoveryUtil


FDisasterRecoverySessionManager::FDisasterRecoverySessionManager(const FString& InRole, TSharedPtr<IConcertSyncClient> InSyncClient)
	: SyncClient(MoveTemp(InSyncClient))
	, Role(InRole)
	, RecoveryInfoPathname(GetDisasterRecoveryInfoPathname())
{
	// Keep recovery working all the time. MessageBus may disconnects the node connected to the server, preventing this client
	// from sending messages. The discovery mechanism forces the client/server to re-handshake and prevent staling the client.
	SyncClient->GetConcertClient()->StartDiscovery();

	// Detects when a session start/end.
	SyncClient->GetConcertClient()->OnSessionStartup().AddRaw(this, &FDisasterRecoverySessionManager::OnRecoverySessionStartup);
	SyncClient->GetConcertClient()->OnSessionShutdown().AddRaw(this, &FDisasterRecoverySessionManager::OnRecoverySessionShutdown);

	// Rotate the sessions and get repositories containing the sessions to discard.
	TArray<FGuid> DiscardedRepositoryIds = InitAndRotateSessions();

	// Fill up the session cache from the recovery info file.
	UpdateSessionsCache();

	auto OnServerFoundFn = [this, DiscardedRepositoryIds = MoveTemp(DiscardedRepositoryIds)](const FGuid& InAdminEndpointId)
	{
		ServerAdminEndpointId = InAdminEndpointId;

		// 2- Discard expired sessions. (Disaster recovery stores only one session per repositories)
		TaskExecutor.Enqueue(MakeShared<FDiscardDisasterRecoveryRepositoriesTask>(
			[this, DiscardedRepositoryIds]() { return SyncClient->GetConcertClient()->DropSessionRepositories(ServerAdminEndpointId, DiscardedRepositoryIds); },
			[this](const TArray<FGuid>& DiscardedRepositoryIds) { OnSessionRepositoriesDiscardedInternal(DiscardedRepositoryIds); }, // Update internal state.
			[](const FText&){})); // Ignore error, this a maintenance task that can rerun at next launch.
	};

	// 1- Resolve the recovery server.
	TaskExecutor.Enqueue(MakeShared<FLookupDisasterRecoveryServerTask>(
		SyncClient->GetConcertClient(),
		SyncClient->GetConcertClient()->GetConfiguration()->DefaultServerURL,
		MoveTemp(OnServerFoundFn), // Callback invoked when the server is discovered.
		[](const FText&){})); // Ignore error, it only reports an error if the task is aborted (when this instance is deleted).

#if WITH_EDITOR
	if (GIsEditor)
	{
		FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>("DirectoryWatcher");
		IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule.Get();

		if (DirectoryWatcher)
		{
			// Watches other Editor instance(s) updating the recovery info file.
			DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(GetDisasterRecoveryInfoPath(), IDirectoryWatcher::FDirectoryChanged::CreateRaw(this, &FDisasterRecoverySessionManager::OnSessionRepositoryFilesModified), DirectoryWatcherHandle);
		}
	}
#endif
}

FDisasterRecoverySessionManager::~FDisasterRecoverySessionManager()
{
	LeaveSession();
	SyncClient->GetConcertClient()->StopDiscovery();

#if WITH_EDITOR
	if (GIsEditor)
	{
		FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>("DirectoryWatcher");
		IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule.Get();
		if (DirectoryWatcher)
		{
			DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(GetSessionRepositoryRootDir(), DirectoryWatcherHandle);
		}
	}
#endif

	// Abort all tasks.
	TaskExecutor.Abort();

	SyncClient->GetConcertClient()->OnSessionStartup().RemoveAll(this);
	SyncClient->GetConcertClient()->OnSessionShutdown().RemoveAll(this);
}

TArray<FGuid> FDisasterRecoverySessionManager::InitAndRotateSessions()
{
	DisasterRecoveryUtil::ScopedRecoveryInfoExclusiveUpdater ScopedRecoveryUpdater(RecoveryInfoPathname, SessionsCacheRevision);
	FDisasterRecoveryInfo& RecoveryInfo = ScopedRecoveryUpdater.Info();

	// If the user is upgrading from 4.24 to 4.25 the name and the format of the recovery info file changed. Try migrating to the new format.
	FString OldSessionInfo = GetDisasterRecoveryInfoPath() / TEXT("Sessions.json");
	if (IFileManager::Get().FileExists(*OldSessionInfo) && RecoveryInfo.Revision == 0)
	{
		DisasterRecoveryUtil::MigrateSessionInfoFrom_4_24(OldSessionInfo, RecoveryInfo);
		IFileManager::Get().Delete(*OldSessionInfo);
	}

	// Remove dead/crashed recovery clients from the list of active clients.
	RecoveryInfo.Clients.RemoveAll([this](const FDisasterRecoveryClientInfo& ClientInfo)
	{
		if (!DisasterRecoveryUtil::IsRecoveryClientProcessRunning(ClientInfo.ClientProcessId))
		{
			// Clean up the temporary directory left behind by this client (it can use lot of disk space).
			FString ClientTempDir = FPaths::ProjectIntermediateDir() / TEXT("Concert") / Role / ClientInfo.ClientAppId.ToString(); // See FConcertClientPaths (inaccessible from here).
			IFileManager::Get().DeleteDirectory(*ClientTempDir, /*bRequireExists*/false, /*Tree*/true);
			return true;
		}
		return false; // The client is sill running, don't remove it.
	});

	// Add this recovery client to the list of active clients.
	RecoveryInfo.Clients.Add(FDisasterRecoveryClientInfo{FPlatformProcess::GetCurrentProcessId(), FApp::GetInstanceId()});
	DisasterRecoveryClientCount = RecoveryInfo.Clients.Num();

	// Unmap repositories that doesn't exist anymore on disk. User might have manually deleted the directories.
	auto UnmapDeletedRepositoriesFn = [&RecoveryInfo](TArray<FDisasterRecoverySession>& Sessions)
	{
		Sessions.RemoveAll([&RecoveryInfo](const FDisasterRecoverySession& Candidate)
		{
			FString Pathname = Candidate.RepositoryRootDir / Candidate.RepositoryId.ToString();
			if (!IFileManager::Get().DirectoryExists(*Pathname))
			{
				RecoveryInfo.DiscardedRepositoryIds.Add(Candidate.RepositoryId); // So that the server can unmap it too.
				return true;
			}
			return false;
		});
	};
	UnmapDeletedRepositoriesFn(ScopedRecoveryUpdater.Info().ActiveSessions);
	UnmapDeletedRepositoriesFn(ScopedRecoveryUpdater.Info().RecentSessions);
	UnmapDeletedRepositoriesFn(ScopedRecoveryUpdater.Info().ImportedSessions);

	// Unmap repositories that were created and mounted, but for which the session was not successfully created (likely because the task was canceled in the process)
	RecoveryInfo.PendingSessions.RemoveAll([&RecoveryInfo](const FDisasterRecoverySession& Pending)
	{
		if (!DisasterRecoveryUtil::IsRecoveryClientProcessRunning(Pending.MountedByProcessId))
		{
			RecoveryInfo.DiscardedRepositoryIds.Add(Pending.RepositoryId);
			return true;
		}
		return false;
	});

	// Update the active sessions, detecting potential crashes.
	for (FDisasterRecoverySession& Active : RecoveryInfo.ActiveSessions)
	{
		check(!EnumHasAnyFlags(static_cast<EDisasterRecoverySessionFlags>(Active.Flags), EDisasterRecoverySessionFlags::Imported | EDisasterRecoverySessionFlags::Recent));

		if (!DisasterRecoveryUtil::IsSessionClientProcessRunning(Active))
		{
			Active.ClientProcessId = 0;
			Active.MountedByProcessId = 0;
			Active.Flags |= static_cast<uint8>(EDisasterRecoverySessionFlags::AbnormalTerminaison); // In the active list but its client is not running -> it is a crash.
		}
	}

	// Update recent and imported session mount states (actives session were already updated above).
	auto ResetMountedStateFn = [](TArray<FDisasterRecoverySession>& SessionList)
	{
		for (FDisasterRecoverySession& Session : SessionList)
		{
			check(Session.ClientProcessId == 0); // Only active sessions can be 'in progress'.

			if (!DisasterRecoveryUtil::IsSessionClientProcessRunning(Session))
			{
				Session.MountedByProcessId = 0;
			}
		}
	};
	ResetMountedStateFn(RecoveryInfo.RecentSessions);
	ResetMountedStateFn(RecoveryInfo.ImportedSessions);

	// Rotate the recents and imported sessions unless another recovery client exists. (that's not nice to the other running instances possibly looking at those sessions).
	if (DisasterRecoveryClientCount == 1)
	{
		auto RotateSessionsFn = [](TArray<FDisasterRecoverySession>& SessionList, TArray<FGuid>& DiscardedList, int32 MaxCount)
		{
			while (SessionList.Num() > MaxCount)
			{
				// Discard the oldest session. (Oldest is at the back of the list).
				DiscardedList.Add(SessionList.Last().RepositoryId);
				SessionList.Pop(EAllowShrinking::No);
			}
		};
		RotateSessionsFn(RecoveryInfo.RecentSessions, RecoveryInfo.DiscardedRepositoryIds, GetRecentSessionMaxCount());
		RotateSessionsFn(RecoveryInfo.ImportedSessions, RecoveryInfo.DiscardedRepositoryIds, GetImportedSessionMaxCount());
	}

	// Return the list of repositories the server must discard. The recovery file will be updated on server response.
	return RecoveryInfo.DiscardedRepositoryIds;
}

void FDisasterRecoverySessionManager::Refresh()
{
	// TODO phase 2: Optimize based on active client count (if only 1 client process exists, no need to refresh)

	// Detect when another instance crashed and update the crashed session state.
	auto UpdateCrashedSessionInfoFn = [this](TArray<FDisasterRecoverySession>& SessionListFromDisk)
	{
		bool bUpdated = false;
		for (FDisasterRecoverySession& SessionFromDisk : SessionListFromDisk)
		{
			if (SessionFromDisk.IsMounted() && !DisasterRecoveryUtil::IsRecoveryClientProcessRunning(SessionFromDisk.MountedByProcessId))
			{
				SessionFromDisk.ClientProcessId = 0;
				SessionFromDisk.MountedByProcessId = 0;
				if (!EnumHasAnyFlags(static_cast<EDisasterRecoverySessionFlags>(SessionFromDisk.Flags), EDisasterRecoverySessionFlags::Recent | EDisasterRecoverySessionFlags::Imported)) // Don't alter the flags of recent/imported sessions.
				{
					SessionFromDisk.Flags |= static_cast<uint8>(EDisasterRecoverySessionFlags::AbnormalTerminaison);
				}
				bUpdated = true;
			}
		}
		return bUpdated;
	};

	bool bSessionUpdated = false;
	uint32 RevisionBefore;
	{
		DisasterRecoveryUtil::ScopedRecoveryInfoExclusiveUpdater ScopedRecoveryUpdater(RecoveryInfoPathname, SessionsCacheRevision);
		FDisasterRecoveryInfo& RecoveryInfo = ScopedRecoveryUpdater.Info();
		RevisionBefore = RecoveryInfo.Revision;

		bSessionUpdated |= UpdateCrashedSessionInfoFn(RecoveryInfo.ActiveSessions);
		bSessionUpdated |= UpdateCrashedSessionInfoFn(RecoveryInfo.RecentSessions);
		bSessionUpdated |= UpdateCrashedSessionInfoFn(RecoveryInfo.ImportedSessions);

		ScopedRecoveryUpdater.SetAutoSave(bSessionUpdated);
	}

	if (bSessionUpdated || RevisionBefore != SessionsCacheRevision) // The file was changed either by the ScopedRecoveryInfoExclusiveUpdater above or by another instance.
	{
		UpdateSessionsCache();
	}
}

void FDisasterRecoverySessionManager::UpdateSessionsCache()
{
	FDisasterRecoveryInfo RecoveryInfo;
	{
		// Get machine-wide exclusive access to the file.
		FSystemWideCriticalSection SystemWideMutex(DisasterRecoveryUtil::GetSystemMutexName());

		// Load the session info file (if it exist)
		DisasterRecoveryUtil::LoadDisasterRecoveryInfo(RecoveryInfoPathname, RecoveryInfo);
		SessionsCacheRevision = RecoveryInfo.Revision;
	}

	// Remove from the cache the sessions that were removed from the recovery info file (possibly by another instance).
	TArray<TSharedRef<FDisasterRecoverySession>> RemovedList;
	SessionsCache.RemoveAll([this, &RecoveryInfo, &RemovedList](const TSharedRef<FDisasterRecoverySession>& RemoveCandidate)
	{
		if (FDisasterRecoverySession* Active = RecoveryInfo.ActiveSessions.FindByPredicate([&RemoveCandidate](const FDisasterRecoverySession& MatchCandidate) { return MatchCandidate.RepositoryId == RemoveCandidate->RepositoryId; }))
		{
			return false;
		}
		else if (FDisasterRecoverySession* Recent = RecoveryInfo.RecentSessions.FindByPredicate([&RemoveCandidate](const FDisasterRecoverySession& MatchCandidate) { return MatchCandidate.RepositoryId == RemoveCandidate->RepositoryId; }))
		{
			return false;
		}
		else if (FDisasterRecoverySession* Imported = RecoveryInfo.ImportedSessions.FindByPredicate([&RemoveCandidate](const FDisasterRecoverySession& MatchCandidate) { return MatchCandidate.RepositoryId == RemoveCandidate->RepositoryId; }))
		{
			return false;
		}

		RemovedList.Add(RemoveCandidate);
		return true; // Remove it.
	});

	// Broadcast the removed item (after they were removed).
	for (const TSharedRef<FDisasterRecoverySession>& Removed : RemovedList)
	{
		OnSessionRemoved().Broadcast(Removed->RepositoryId);
	}

	// Add to or update the cache for sessions that were added/updated in the recovery info file (possibly by another instance).
	auto UpdateSessionCacheFn = [this](const TArray<FDisasterRecoverySession>& ListFromDisk)
	{
		for (const FDisasterRecoverySession& SessionFromFile : ListFromDisk)
		{
			if (TSharedRef<FDisasterRecoverySession>* SessionFromCache = SessionsCache.FindByPredicate([&SessionFromFile](const TSharedRef<FDisasterRecoverySession>& Candidate) { return Candidate->RepositoryId == SessionFromFile.RepositoryId; }))
			{
				if (**SessionFromCache != SessionFromFile) // Session was updated?
				{
					**SessionFromCache = SessionFromFile; // Update the cached value.
					OnSessionUpdated().Broadcast(*SessionFromCache);
				}
			}
			else // Discovered a new session.
			{
				OnSessionAdded().Broadcast(SessionsCache.Add_GetRef(MakeShared<FDisasterRecoverySession>(SessionFromFile)));
			}
		}
	};

	// Update the cache by comparing it to the sessions on disk.
	UpdateSessionCacheFn(RecoveryInfo.ActiveSessions);
	UpdateSessionCacheFn(RecoveryInfo.RecentSessions);
	UpdateSessionCacheFn(RecoveryInfo.ImportedSessions);
}

TFuture<bool> FDisasterRecoverySessionManager::HasRecoverableCandidates()
{
	TSharedPtr<TPromise<bool>> Promise = MakeShared<TPromise<bool>>();

	auto OnErrorFn = [Promise](const FText& ErrorMsg)
	{
		// If an error occurred during the process log it, something is likely wrong with the implementation, returning a 'no candidate found' is the best return value in this case.
		UE_LOGF(LogDisasterRecovery, Error, "Unexpected error occurred when looking for recoverable candidate. Error: %ls", *ErrorMsg.ToString());
		Promise->SetValue(false);
	};

	auto OnCandidateSearchResultFn = [this, Promise](TSharedPtr<FDisasterRecoverySession> Candidate)
	{
		Promise->SetValue(Candidate != nullptr);
	};

	// 1 - List the possible candidates.
	const bool bIgnoreDebuggerFlag = false;
	TArray<TSharedPtr<FDisasterRecoverySession>> Candidates;
	for (const TSharedPtr<FDisasterRecoverySession> Candidate : SessionsCache)
	{
		if (Candidate->WasDebuggerAttached() && !bIgnoreDebuggerFlag)
		{
			continue; // Don't prompt the user to recover when the debugger was attached. User likely stopped the debugging in the middle of a session and is likely not interested to recover.
		}

		// Unreviewed crashes are obvious candidates. For a live session (owner process is still alive), if its repository cannot be mounted by this process, it will be eliminated, but if can,
		// it means that the 'out-of-process' crash reporter caught a crash and unmounted the session repository even if the owned process is still active from OS point of view.
		if (Candidate->IsUnreviewedCrash() || (Candidate->IsLive() && Candidate->ClientProcessId != FPlatformProcess::GetCurrentProcessId()))
		{
			Candidates.Add(Candidate);
		}
	}

	if (Candidates.Num() == 0)
	{
		OnCandidateSearchResultFn(nullptr); // No candidate found.
	}
	else
	{
		// 2- Verify if at least one candidates can be mounted and has activities to recover.
		TaskExecutor.Enqueue(MakeShared<FFindRecoverableDisasterRecoverySessionTask>(
			SyncClient->GetConcertClient(),
			[this]() { return ServerAdminEndpointId; }, // As a function in case this task is enqueue before the server was found.
			Candidates,
			OnCandidateSearchResultFn,
			OnErrorFn));
	}

	return Promise->GetFuture();
}

TFuture<TVariant<TSharedPtr<FConcertActivityStream>, FText>> FDisasterRecoverySessionManager::LoadSession(TSharedRef<FDisasterRecoverySession> Session)
{
	TSharedPtr<TPromise<TVariant<TSharedPtr<FConcertActivityStream>, FText>>> Promise = MakeShared<TPromise<TVariant<TSharedPtr<FConcertActivityStream>, FText>>>();

	auto OnErrorFn = [Promise](const FText& ErrorMsg)
	{
		Promise->EmplaceValue(TInPlaceType<FText>(), ErrorMsg); // Could not load the session activities.
	};

	auto OnSessionIdLookupResultFn = [this, OnErrorFn, Session, Promise](const FGuid& ResultSessionId)
	{
		if (!ResultSessionId.IsValid()) // Not found?
		{
			OnSessionNotFoundInternal(Session->RepositoryId); // The session was likely deleted. Mark it to be discarded to be cleaned up at the next start up.
			OnErrorFn(LOCTEXT("SessionNotFound", "Session not found."));
			return; // 3b - No session.
		}

		// 3a- Return the activity stream on the session.
		TSharedPtr<FConcertActivityStream> ActivityStream = MakeShared<FConcertActivityStream>(SyncClient->GetConcertClient(), ServerAdminEndpointId, ResultSessionId, /*bInIncludeActivityDetails*/true);
		return Promise->EmplaceValue(TInPlaceType<TSharedPtr<FConcertActivityStream>>(), ActivityStream);
	};

	auto OnRepositoryMountedFn = [this, Session, OnErrorFn, OnSessionIdLookupResultFn]()
	{
		OnSessionRepositoryMountedInternal(Session->RepositoryId); // Update internal state.

		// 2- Find the session by name and retrieve its ID.
		TaskExecutor.Enqueue(MakeShared<FLookupDisasterRecoverySessionIdTask>(
			[this](){ return SyncClient->GetConcertClient()->GetServerSessions(ServerAdminEndpointId); },
			Session->SessionName,
			OnSessionIdLookupResultFn,
			OnErrorFn));
	};

	// 1- Mount the session repository, the server will reload the session in memory. (Create the mount point if it doesn't exist in case its for an imported session the server has not seen yet)
	TaskExecutor.Enqueue(MakeShared<FMountDisasterRecoverySessionRepositoryTask>(
		[this, Session]() { return SyncClient->GetConcertClient()->MountSessionRepository(ServerAdminEndpointId, Session->RepositoryRootDir, Session->RepositoryId, /*bCreateIfNotExist*/true, /*bAsDefault*/false); },
		OnRepositoryMountedFn,
		OnErrorFn));

	return Promise->GetFuture();
}

TVariant<TSharedPtr<FDisasterRecoverySession>, FText> FDisasterRecoverySessionManager::ImportSession(const FString& SessionInfoPathname)
{
	static const FString SessionInfoFilename = TEXT("SessionInfo.json");
	static const FString SessionDatabaseFilename = TEXT("Session.db");
	static const FString SessionDatabaseWalFilename = TEXT("Session.db-wal"); // The Write-ahead-log file, if present, likely contains some DB transactions.
	//static const FString TransactionsDirName = TEXT("Transactions");
	//static const FString PackagesDirName = TEXT("Packages");

	FText ErrorMsg;
	TSharedPtr<FDisasterRecoverySession> ImportedSession;

	// Load the session info in memory.
	FConcertSessionInfo SessionInfo;
	TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*SessionInfoPathname));
	if (FileReader)
	{
		FJsonStructDeserializerBackend Backend(*FileReader);
		FStructDeserializer::Deserialize(SessionInfo, Backend);
		FileReader->Close();
	}

	// Copy the files in the archived session dir.
	FString SessionDatabaseSrcPathname = FPaths::GetPath(SessionInfoPathname) / SessionDatabaseFilename;
	FString SessionDatabaseWalSrcPathname = FPaths::GetPath(SessionInfoPathname) / SessionDatabaseWalFilename; // This file is not always there, depending on the database state.

	if (SessionInfo.SessionName.IsEmpty())
	{
		ErrorMsg = FText::Format(LOCTEXT("InvalidSessionName", "Could not parse the session name from '{0}'."), FText::AsCultureInvariant(SessionInfoPathname));
	}
	else if (!SessionInfo.SessionId.IsValid())
	{
		ErrorMsg = FText::Format(LOCTEXT("InvalidSessionId", "Could not parse the session ID from '{0}'."), FText::AsCultureInvariant(SessionInfoPathname));
	}
	else if (!IFileManager::Get().FileExists(*SessionDatabaseSrcPathname)) // Session.db must exist aside SessionInfo.json
	{
		ErrorMsg = FText::Format(LOCTEXT("ActivityDatabaseNotFound", "Could not find '{0}' in '{1}'."), FText::AsCultureInvariant(SessionDatabaseFilename), FText::AsCultureInvariant(FPaths::GetPath(SessionInfoPathname)));
	}
	else
	{
		ImportedSession = MakeShared<FDisasterRecoverySession>();
		ImportedSession->Flags = static_cast<uint8>(EDisasterRecoverySessionFlags::Imported);
		ImportedSession->SessionName = SessionInfo.SessionName;
		ImportedSession->RepositoryId = FGuid::NewGuid();
		ImportedSession->RepositoryRootDir = GetSessionRepositoryRootDir();

		// Create a new repository to contain the session (to be mounted by the server later) in the repository root dir.
		const FString RepositoryDir = ImportedSession->RepositoryRootDir / ImportedSession->RepositoryId.ToString();
		const FString SessionDir = RepositoryDir / TEXT("Archive") / SessionInfo.SessionId.ToString();

		// Make the session specific directory
		if (IFileManager::Get().MakeDirectory(*SessionDir, /*Tree*/true))
		{
			// Copy the files in the SessionDir.
			FString SessionInfoDestPathname = SessionDir / SessionInfoFilename;
			FString SessionDatabaseDestPathname = SessionDir / SessionDatabaseFilename;
			FString SessionDatabaseWalDestPathname = SessionDir / SessionDatabaseWalFilename;
			if (IFileManager::Get().Copy(*SessionInfoDestPathname, *SessionInfoPathname) != COPY_OK)
			{
				ErrorMsg = FText::Format(LOCTEXT("FailedToCopySessionInfo", "Failed to copy '{0}' to '{1}'."), FText::AsCultureInvariant(SessionInfoPathname), FText::AsCultureInvariant(SessionDir));
				ImportedSession.Reset();
			}
			else if (IFileManager::Get().Copy(*SessionDatabaseDestPathname, *SessionDatabaseSrcPathname) != COPY_OK)
			{
				ErrorMsg = FText::Format(LOCTEXT("FailedToCopySessionDB", "Failed to copy '{0}' to '{1}'."), FText::AsCultureInvariant(SessionDatabaseSrcPathname), FText::AsCultureInvariant(SessionDir));
				ImportedSession.Reset();
			}
			else if (IFileManager::Get().FileExists(*SessionDatabaseWalSrcPathname) && IFileManager::Get().Copy(*SessionDatabaseWalDestPathname, *SessionDatabaseWalSrcPathname) != COPY_OK)
			{
				ErrorMsg = FText::Format(LOCTEXT("FailedToCopySessionDBWal", "Failed to copy '{0}' to '{1}'."), FText::AsCultureInvariant(SessionDatabaseWalSrcPathname), FText::AsCultureInvariant(SessionDir));
				ImportedSession.Reset();
			}
			else
			{
				// TODO phase 2: Copy Transactions folder (if any)
				// TODO phase 2: Copy Packages folder (if any)
				OnSessionImportedInternal(ImportedSession.ToSharedRef());
			}
		}
		else
		{
			ErrorMsg = FText::Format(LOCTEXT("FailedToCreateImportDir", "Failed to create import directory {0}."), FText::AsCultureInvariant(SessionDir));
		}
	}

	TVariant<TSharedPtr<FDisasterRecoverySession>, FText> Result;
	if (ErrorMsg.IsEmpty() && ImportedSession)
	{
		Result.Set<TSharedPtr<FDisasterRecoverySession>>(ImportedSession);
	}
	else
	{
		Result.Set<FText>(ErrorMsg);
	}
	return Result;
}

TFuture<TPair<bool,FText>> FDisasterRecoverySessionManager::DiscardSession(const FDisasterRecoverySession& Session)
{
	// Discarding a session is equivalent to discarding its repository. By design, each session has its own repository.
	return DiscardRepositories(TArray<FGuid>{ Session.RepositoryId });
}

TFuture<TPair<bool,FText>> FDisasterRecoverySessionManager::DiscardRepositories(TArray<FGuid> RepositoryIds)
{
	TSharedPtr<TPromise<TPair<bool, FText>>> Promise = MakeShared<TPromise<TPair<bool, FText>>>();

	auto OnSessionsDiscardedFn = [this, Promise](const TArray<FGuid>& DiscardedRepositoryIds)
	{
		// 2- Update the local states
		OnSessionRepositoriesDiscardedInternal(DiscardedRepositoryIds); // Update internal state.
		Promise->SetValue(MakeTuple(true, FText()));
	};

	auto OnErrorFn = [Promise](const FText& ErrorMsg)
	{
		Promise->SetValue(MakeTuple(false, ErrorMsg));
	};

	// 1- Ask the server to discard the repositories.
	TaskExecutor.Enqueue(MakeShared<FDiscardDisasterRecoveryRepositoriesTask>(
		[this, RepositoryIds = MoveTemp(RepositoryIds)]() { return SyncClient->GetConcertClient()->DropSessionRepositories(ServerAdminEndpointId, RepositoryIds); },
		OnSessionsDiscardedFn,
		OnErrorFn));

	return Promise->GetFuture();
}

TFuture<TPair<bool, FText>> FDisasterRecoverySessionManager::CreateAndJoinSession()
{
	// Failed if another session is already in progress or active.
	if (HasInProgressSession())
	{
		return MakeFulfilledPromise<TPair<bool, FText>>(MakeTuple(false, LOCTEXT("FailedToCreateAndJoin", "Failed to create the session. Another session is already in progress"))).GetFuture();
	}

	TSharedPtr<TPromise<TPair<bool, FText>>> Promise = MakeShared<TPromise<TPair<bool, FText>>>();
	FString SessionName = RecoveryService::MakeSessionName(); // This follow the convention used in CrachReportClientEditor.
	FGuid RepositoryId = FGuid::NewGuid(); // The repository for this session.

	// Remember the current session info.
	CurrentSessionRepositoryId = RepositoryId;
	CurrentSessionName = SessionName;

	auto OnErrorFn = [this, Promise, RepositoryId](const FText& ErrorMsg)
	{
		if (RepositoryId == CurrentSessionRepositoryId) // Wasn't aborted.
		{
			CurrentSessionRepositoryId.Invalidate();
			CurrentSessionName.Reset();
			Promise->SetValue(MakeTuple(false, ErrorMsg));
		}
		else // Failed, but was aborted first.
		{
			UE_LOGF(LogDisasterRecovery, Verbose, "CreateAndJoinSession Aborted (OnErrorFn). RepositoryID: %ls", *RepositoryId.ToString());
			Promise->SetValue(MakeTuple(false, DisasterRecoveryUtil::GetOperationAbortedMessage()));
		}
	};

	auto OnSessionJoinedFn = [this, Promise, RepositoryId]()
	{
		// Note: OnRecoverySessionStartup() is expected to be called first and should have processed the new session.
		if (RepositoryId == CurrentSessionRepositoryId) // Wasn't aborted.
		{
			Promise->SetValue(MakeTuple(true, FText()));
		}
		else // Was aborted while creating/joining.
		{
			UE_LOGF(LogDisasterRecovery, Verbose, "CreateAndJoinSession Aborted (OnSessionJoinedFn). RepositoryID: %ls", *RepositoryId.ToString());
			Promise->SetValue(MakeTuple(false, DisasterRecoveryUtil::GetOperationAbortedMessage()));
		}
	};

	auto OnRepositoryMountedFn = [this, Promise, SessionName, RepositoryId, OnSessionJoinedFn, OnErrorFn]()
	{
		OnSessionRepositoryMountedInternal(RepositoryId); // Update internal state.

		if (RepositoryId != CurrentSessionRepositoryId) // Aborted?
		{
			UE_LOGF(LogDisasterRecovery, Verbose, "CreateAndJoinSession Aborted (OnRepositoryMountedFn). RepositoryID: %ls", *RepositoryId.ToString());
			Promise->SetValue(MakeTuple(false, DisasterRecoveryUtil::GetOperationAbortedMessage()));
			return;
		}

		// Create and join the session.
		FConcertCreateSessionArgs CreateArgs;
		IConcertClientRef Client = SyncClient->GetConcertClient();
		const UConcertClientConfig* ClientConfig = Client->GetConfiguration();
		CreateArgs.SessionName = SessionName;
		CreateArgs.ArchiveNameOverride = SessionName;

		// 2- Create and join a new session.
		TaskExecutor.Enqueue(MakeShared<FCreateAndJoinDisasterRecoverySessionTask>(
			[this, CreateArgs = MoveTemp(CreateArgs)](){ return SyncClient->GetConcertClient()->CreateSession(ServerAdminEndpointId, CreateArgs); },
			SyncClient,
			SessionName,
			OnSessionJoinedFn,
			OnErrorFn));
	};

	// 1- Mount the session repository, set it as default to ensure the new session is going to be created in it.
	TaskExecutor.Enqueue(MakeShared<FMountDisasterRecoverySessionRepositoryTask>(
		[this]() { return SyncClient->GetConcertClient()->MountSessionRepository(ServerAdminEndpointId, GetSessionRepositoryRootDir(), CurrentSessionRepositoryId, /*bCreateIfNotExist*/true, /*bAsDefault*/true); },
		OnRepositoryMountedFn,
		OnErrorFn));

	return Promise->GetFuture();
}

TFuture<TPair<bool, FText>> FDisasterRecoverySessionManager::RestoreAndJoinSession(TSharedPtr<FDisasterRecoverySession> Session, TSharedPtr<FConcertSessionActivity> ThroughActivity)
{
	// Failed if another session is already in progress or active.
	if (HasInProgressSession())
	{
		return MakeFulfilledPromise<TPair<bool, FText>>(MakeTuple(false, LOCTEXT("FailedToRestoreAndJoin", "Failed to restore the session. Another session is already active."))).GetFuture();
	}

	TSharedPtr<TPromise<TPair<bool, FText>>> Promise = MakeShared<TPromise<TPair<bool, FText>>>();
	FString SessionName = RecoveryService::MakeSessionName(); // This follow the convention used in CrachReportClientEditor.
	FGuid RepositoryId = FGuid::NewGuid(); // The repository for this session.
	TSharedPtr<FGuid> SrcSessionId = MakeShared<FGuid>(); // Will hold the ID of the session to restore (once found).

	// Remember the current session info.
	CurrentSessionRepositoryId = RepositoryId;
	CurrentSessionName = SessionName;

	auto OnErrorFn = [this, Promise, RepositoryId](const FText& ErrorMsg)
	{
		if (RepositoryId == CurrentSessionRepositoryId) // Wasn't aborted.
		{
			CurrentSessionRepositoryId.Invalidate();
			CurrentSessionName.Reset();
			Promise->SetValue(MakeTuple(false, ErrorMsg));
		}
		else // Failed, but was aborted first.
		{
			Promise->SetValue(MakeTuple(false, DisasterRecoveryUtil::GetOperationAbortedMessage()));
		}
	};

	auto OnSessionRestoredFn = [this, Promise, RepositoryId]()
	{
		if (RepositoryId == CurrentSessionRepositoryId) // Wasn't aborted.
		{
			// 5 - Persist the local changes and signal completion to the client.
			UE_LOGF(LogDisasterRecovery, Verbose, "Persisting recovered changes...");
			SyncClient->PersistAllSessionChanges();
			UE_LOGF(LogDisasterRecovery, Verbose, "...Persisting recovered changes -> Done");
			Promise->SetValue(MakeTuple(true, FText()));
			// Note: The restored session is added and notified in OnRecoverySessionStartup().
		}
		else
		{
			Promise->SetValue(MakeTuple(false, DisasterRecoveryUtil::GetOperationAbortedMessage()));
		}
	};

	auto OnNewSessionRepositoryMountedFn = [this, Promise, RepositoryId, ThroughActivity, SessionName, SrcSessionId, OnSessionRestoredFn, OnErrorFn]()
	{
		OnSessionRepositoryMountedInternal(RepositoryId); // Update internal state.

		if (RepositoryId != CurrentSessionRepositoryId) // Was aborted?
		{
			Promise->SetValue(MakeTuple(false, DisasterRecoveryUtil::GetOperationAbortedMessage()));
			return;
		}

		FConcertCopySessionArgs CopyArgs;
		CopyArgs.ArchiveNameOverride = SessionName;
		CopyArgs.bAutoConnect = true; // Join the session once copied.
		CopyArgs.SessionId = *SrcSessionId; // The session to copy.
		CopyArgs.SessionFilter.ActivityIdUpperBound = ThroughActivity->Activity.ActivityId;
		CopyArgs.SessionFilter.bOnlyLiveData = false; // Reapply all activities.
		CopyArgs.SessionFilter.bIncludeIgnoredActivities = false; // Don't restore ignored activities. (Like MultiUser activities recorded in a DisasterRecovery session)
		CopyArgs.SessionName = SessionName;

		// 4 - Create a new session from the source one and join it.
		TaskExecutor.Enqueue(MakeShared<FCreateAndJoinDisasterRecoverySessionTask>(
			[this, CopyArgs = MoveTemp(CopyArgs)](){ return SyncClient->GetConcertClient()->CopySession(ServerAdminEndpointId, CopyArgs); },
			SyncClient,
			SessionName,
			OnSessionRestoredFn,
			OnErrorFn));
	};

	auto OnSessionIdLookupResultFn = [this, Promise, RepositoryId, SrcSessionId, Session, OnNewSessionRepositoryMountedFn, OnErrorFn](const FGuid& InSrcSessionId)
	{
		if (RepositoryId != CurrentSessionRepositoryId) // Was aborted?
		{
			Promise->SetValue(MakeTuple(false, DisasterRecoveryUtil::GetOperationAbortedMessage()));
			return;
		}

		if (!InSrcSessionId.IsValid()) // Not found?
		{
			OnSessionNotFoundInternal(Session->RepositoryId); // The session was likely deleted. Mark its repository to be discarded at the next start up.
			OnErrorFn(LOCTEXT("SessionNotFound", "Session not found."));
			return;
		}

		*SrcSessionId = InSrcSessionId; // Store the session ID for the next task above.

		// 3- Mount a new repository to contain the new session (a new session is created from the one to restore).
		TaskExecutor.Enqueue(MakeShared<FMountDisasterRecoverySessionRepositoryTask>(
			[this, RepositoryId]() { return SyncClient->GetConcertClient()->MountSessionRepository(ServerAdminEndpointId, GetSessionRepositoryRootDir(), RepositoryId, /*bCreateIfNotExist*/true, /*bAsDefault*/true); },
			OnNewSessionRepositoryMountedFn,
			OnErrorFn));
	};

	auto OnSrcRepositoryMountedFn = [this, Promise, RepositoryId, Session, OnErrorFn, OnSessionIdLookupResultFn]()
	{
		OnSessionRepositoryMountedInternal(Session->RepositoryId); // The session repository was mounted. Update internal state.

		if (RepositoryId != CurrentSessionRepositoryId) // Was aborted?
		{
			Promise->SetValue(MakeTuple(false, DisasterRecoveryUtil::GetOperationAbortedMessage()));
			return;
		}

		// 2- Search the session by name and retreive its ID.
		TaskExecutor.Enqueue(MakeShared<FLookupDisasterRecoverySessionIdTask>(
			[this](){ return SyncClient->GetConcertClient()->GetServerSessions(ServerAdminEndpointId); },
			Session->SessionName,
			OnSessionIdLookupResultFn,
			OnErrorFn));
	};

	// 1- Mount the repository containing the session to restore, the server will reload the sessions in that repository.
	TaskExecutor.Enqueue(MakeShared<FMountDisasterRecoverySessionRepositoryTask>(
		[this, Session]() { return SyncClient->GetConcertClient()->MountSessionRepository(ServerAdminEndpointId, Session->RepositoryRootDir, Session->RepositoryId, /*bCreateIfNotExist*/false, /*bAsDefault*/false); },
		OnSrcRepositoryMountedFn,
		OnErrorFn));

	return Promise->GetFuture();
}

void FDisasterRecoverySessionManager::LeaveSession()
{
	// If a task chain was started to create/restore a session.
	if (HasInProgressSession())
	{
		UE_LOGF(LogDisasterRecovery, Verbose, "Leaving session '%ls'. RepositoryID: %ls)", *CurrentSessionName, *CurrentSessionRepositoryId.ToString());

		// Disconnect from the session (if the connection started).
		SyncClient->GetConcertClient()->DisconnectSession();

		// Clear up the current active state.
		CurrentSessionRepositoryId.Invalidate();
		CurrentSessionName.Reset();
	}
}

bool FDisasterRecoverySessionManager::HasInProgressSession() const
{
	return CurrentSessionRepositoryId.IsValid();
}

void FDisasterRecoverySessionManager::OnRecoverySessionStartup(TSharedRef<IConcertClientSession> InSession)
{
	{
		DisasterRecoveryUtil::ScopedRecoveryInfoExclusiveUpdater ScopedRecoveryUpdater(RecoveryInfoPathname, SessionsCacheRevision);
		FDisasterRecoveryInfo& RecoveryInfo = ScopedRecoveryUpdater.Info();

		// The user had the chance to 'review' the 'pending' candidates, move them to the 'recent' list to not review them again.
		RecoveryInfo.ActiveSessions.RemoveAll([&RecoveryInfo](const FDisasterRecoverySession& Candidate)
		{
			if (Candidate.ClientProcessId == 0 && (Candidate.MountedByProcessId == 0 || Candidate.MountedByProcessId == FPlatformProcess::GetCurrentProcessId())) // Not used by another instance.
			{
				// Insert the session to the recent list. (Newest are at the front)
				RecoveryInfo.RecentSessions.Insert(Candidate, 0);
				RecoveryInfo.RecentSessions[0].Flags |= static_cast<uint8>(EDisasterRecoverySessionFlags::Recent);
				return true; // Yes, remove it from the list of active session.
			}

			return false; // Keep it active, this session is locked by another instance.
		});

		// That is the expected session.
		if (CurrentSessionName == InSession->GetSessionInfo().SessionName)
		{
			UE_LOGF(LogDisasterRecovery, Verbose, "Recovery session '%ls' started. RepositoryID: %ls)", *CurrentSessionName, *CurrentSessionRepositoryId.ToString());

			// Store information about the newly created/restored session.
			FDisasterRecoverySession& RecoverySession = RecoveryInfo.ActiveSessions.AddDefaulted_GetRef();
			RecoverySession.RepositoryRootDir = GetSessionRepositoryRootDir();
			RecoverySession.SessionName = InSession->GetSessionInfo().SessionName;
			RecoverySession.MountedByProcessId = FPlatformProcess::GetCurrentProcessId();
			RecoverySession.ClientProcessId = FPlatformProcess::GetCurrentProcessId();
			RecoverySession.RepositoryId = CurrentSessionRepositoryId;
			RecoverySession.Flags = FPlatformMisc::IsDebuggerPresent() ? static_cast<uint8>(EDisasterRecoverySessionFlags::DebuggerAttached) : static_cast<uint8>(EDisasterRecoverySessionFlags::None);
 
			// Session was created, remove it from the pending list.
			RecoveryInfo.PendingSessions.RemoveAll([this](const FDisasterRecoverySession& RemoveCandidate) { return RemoveCandidate.RepositoryId == CurrentSessionRepositoryId; });
		}
	}

	// Refresh the cached session and notify updates.
	UpdateSessionsCache();
}

void FDisasterRecoverySessionManager::OnRecoverySessionShutdown(TSharedRef<IConcertClientSession> InSession)
{
	{
		DisasterRecoveryUtil::ScopedRecoveryInfoExclusiveUpdater ScopedRecoveryUpdater(RecoveryInfoPathname, SessionsCacheRevision);
		FDisasterRecoveryInfo& RecoveryInfo = ScopedRecoveryUpdater.Info();

		// The session ended up normally, move the current session from the 'active' list to the 'recent' list.
		int32 ThisProcessId = FPlatformProcess::GetCurrentProcessId();
		RecoveryInfo.ActiveSessions.RemoveAll([this, ThisProcessId, &RecoveryInfo, &ScopedRecoveryUpdater, &InSession](const FDisasterRecoverySession& Session)
		{
			if (Session.ClientProcessId == ThisProcessId)
			{
				if (Session.RepositoryId == CurrentSessionRepositoryId) // Is this the async callback for the current session?
				{
					CurrentSessionRepositoryId.Invalidate();
					CurrentSessionName.Reset();
				}

				// Add the most recent session at the front of the recent list.
				RecoveryInfo.RecentSessions.Insert(Session, 0);
				RecoveryInfo.RecentSessions[0].ClientProcessId = 0;
				RecoveryInfo.RecentSessions[0].Flags = static_cast<uint8>(EDisasterRecoverySessionFlags::Recent);
				if (EnumHasAnyFlags(static_cast<EDisasterRecoverySessionFlags>(Session.Flags), EDisasterRecoverySessionFlags::DebuggerAttached))
				{
					RecoveryInfo.RecentSessions[0].Flags |= static_cast<uint8>(EDisasterRecoverySessionFlags::DebuggerAttached);
				}
				UE_LOGF(LogDisasterRecovery, Verbose, "Recovery session '%ls' was shutdown normally", *InSession->GetSessionInfo().SessionName);
				return true; // Remove from the active list.
			}
			return false; // Keep it in the active list.
		});
	}

	UpdateSessionsCache();
}

void FDisasterRecoverySessionManager::OnSessionNotFoundInternal(const FGuid& RepositoryId)
{
	{
		DisasterRecoveryUtil::ScopedRecoveryInfoExclusiveUpdater ScopedRecoveryUpdater(RecoveryInfoPathname, SessionsCacheRevision);
		FDisasterRecoveryInfo& RecoveryInfo = ScopedRecoveryUpdater.Info();

		// Mark the session repository to be discarded.
		RecoveryInfo.ActiveSessions.RemoveAll([RepositoryId](const FDisasterRecoverySession& Candidate) { return RepositoryId == Candidate.RepositoryId; });
		RecoveryInfo.RecentSessions.RemoveAll([RepositoryId](const FDisasterRecoverySession& Candidate) { return RepositoryId == Candidate.RepositoryId; });
		RecoveryInfo.ImportedSessions.RemoveAll([RepositoryId](const FDisasterRecoverySession& Candidate) { return RepositoryId == Candidate.RepositoryId; });
		RecoveryInfo.DiscardedRepositoryIds.Add(RepositoryId);
	}

	// Update the session cache and fire a notification.
	if (SessionsCache.RemoveAll([RepositoryId](const TSharedRef<FDisasterRecoverySession>& Candidate) { return RepositoryId == Candidate->RepositoryId; }))
	{
		OnSessionRemoved().Broadcast(RepositoryId);
	}
}

void FDisasterRecoverySessionManager::OnSessionRepositoriesDiscardedInternal(const TArray<FGuid>& DiscardedRepositoryIds)
{
	{
		DisasterRecoveryUtil::ScopedRecoveryInfoExclusiveUpdater ScopedRecoveryUpdater(RecoveryInfoPathname, SessionsCacheRevision);
		FDisasterRecoveryInfo& RecoveryInfo = ScopedRecoveryUpdater.Info();

		// If the repository was discarded, the session cannot be restored or visualized anymore, clear it from the session database.
		for (const FGuid& RepositoryId : DiscardedRepositoryIds)
		{
			RecoveryInfo.ActiveSessions.RemoveAll([RepositoryId](const FDisasterRecoverySession& Candidate) { return RepositoryId == Candidate.RepositoryId; });
			RecoveryInfo.RecentSessions.RemoveAll([RepositoryId](const FDisasterRecoverySession& Candidate) { return RepositoryId == Candidate.RepositoryId; });
			RecoveryInfo.ImportedSessions.RemoveAll([RepositoryId](const FDisasterRecoverySession& Candidate) { return RepositoryId == Candidate.RepositoryId; });
			RecoveryInfo.DiscardedRepositoryIds.RemoveAll([RepositoryId](const FGuid& Candidate) { return RepositoryId == Candidate; });
		}
	}

	// Update the session cache and broadcast the changes.
	for (const FGuid& RepositoryId : DiscardedRepositoryIds)
	{
		if (SessionsCache.RemoveAll([RepositoryId](const TSharedRef<FDisasterRecoverySession>& Candidate) { return RepositoryId == Candidate->RepositoryId; }))
		{
			OnSessionRemoved().Broadcast(RepositoryId);
		}
	}
}

void FDisasterRecoverySessionManager::OnSessionRepositoryMountedInternal(const FGuid& RepositoryId)
{
	int32 ThisProcessId = FPlatformProcess::GetCurrentProcessId();

	{
		DisasterRecoveryUtil::ScopedRecoveryInfoExclusiveUpdater ScopedRecoveryUpdater(RecoveryInfoPathname, SessionsCacheRevision);
		FDisasterRecoveryInfo& RecoveryInfo = ScopedRecoveryUpdater.Info();

		if (FDisasterRecoverySession* Active = RecoveryInfo.ActiveSessions.FindByPredicate([&RepositoryId](const FDisasterRecoverySession& Candidate) { return Candidate.RepositoryId == RepositoryId; }))
		{
			Active->MountedByProcessId = ThisProcessId;
		}
		else if (FDisasterRecoverySession* Recent = RecoveryInfo.RecentSessions.FindByPredicate([&RepositoryId](const FDisasterRecoverySession& Candidate) { return Candidate.RepositoryId == RepositoryId; }))
		{
			Recent->MountedByProcessId = ThisProcessId;
		}
		else if (FDisasterRecoverySession* Imported = RecoveryInfo.ImportedSessions.FindByPredicate([&RepositoryId](const FDisasterRecoverySession& Candidate) { return Candidate.RepositoryId == RepositoryId; }))
		{
			Imported->MountedByProcessId = ThisProcessId;
		}
		else // A repository was created and mounted to host a session, but the session is not created yet. Track it to prevent 'leak' in case the creation fails or is aborted.
		{
			FDisasterRecoverySession PendingSession;
			PendingSession.RepositoryId = RepositoryId;
			PendingSession.MountedByProcessId = ThisProcessId;
			RecoveryInfo.PendingSessions.Add(MoveTemp(PendingSession));
		}
	}

	// Update the cache and fire the update notification.
	if (TSharedRef<FDisasterRecoverySession>* UpdatedSession = SessionsCache.FindByPredicate([RepositoryId](const TSharedRef<FDisasterRecoverySession>& Candidate) { return Candidate->RepositoryId == RepositoryId; }))
	{
		if ((*UpdatedSession)->MountedByProcessId != ThisProcessId) // Wasn't already mounted by this process?
		{
			(*UpdatedSession)->MountedByProcessId = ThisProcessId;
			OnSessionUpdated().Broadcast((*UpdatedSession));
		}
	}
}

void FDisasterRecoverySessionManager::OnSessionImportedInternal(const TSharedRef<FDisasterRecoverySession>& ImportedSession)
{
	{
		DisasterRecoveryUtil::ScopedRecoveryInfoExclusiveUpdater ScopedRecoveryUpdater(RecoveryInfoPathname, SessionsCacheRevision);
		ScopedRecoveryUpdater.Info().ImportedSessions.Insert(*ImportedSession, 0); // Most recently import are at front.
	}

	// Update the session cache and broadcast the changes.
	SessionsCache.Add(ImportedSession);
	OnSessionAdded().Broadcast(ImportedSession);
}

FString FDisasterRecoverySessionManager::GetSessionRepositoryRootDir() const
{
	const FString& RootDir = GetDefault<UDisasterRecoverClientConfig>()->RecoverySessionDir.Path;
	if (!RootDir.IsEmpty() && (IFileManager::Get().DirectoryExists(*RootDir) || IFileManager::Get().MakeDirectory(*RootDir, /*Tree*/true)))
	{
		return RootDir;
	}

	return FPaths::ProjectSavedDir() / Role / TEXT("Sessions"); // Returns the default.
}

void FDisasterRecoverySessionManager::OnSessionRepositoryFilesModified(const TArray<FFileChangeData>& InFileChanges)
{
	for (const FFileChangeData& FileChangeData : InFileChanges)
	{
		if (FileChangeData.Action == FFileChangeData::EFileChangeAction::FCA_Modified && FileChangeData.Filename.EndsWith(GetDisasterRecoveryInfoFilename()))
		{
			if (DisasterRecoveryUtil::GetDisasterRecoveryInfoVersion(RecoveryInfoPathname) != SessionsCacheRevision) // Skip refresh is it wasn't modified by another process.
			{
				Refresh();
			}
			return; // Don't process the file more than once per notification.
		}
	}
}

FString FDisasterRecoverySessionManager::GetDisasterRecoveryInfoPath() const
{
	return FPaths::ProjectSavedDir() / Role;
}

const TCHAR* FDisasterRecoverySessionManager::GetDisasterRecoveryInfoFilename() const
{
	return TEXT("RecoveryInfo.json");
}

FString FDisasterRecoverySessionManager::GetDisasterRecoveryInfoPathname() const
{
	return GetDisasterRecoveryInfoPath() / GetDisasterRecoveryInfoFilename();
}

int32 FDisasterRecoverySessionManager::GetRecentSessionMaxCount() const
{
	return FMath::Max(0, GetDefault<UDisasterRecoverClientConfig>()->RecentSessionMaxCount);
}

int32 FDisasterRecoverySessionManager::GetImportedSessionMaxCount() const
{
	return FMath::Max(0, GetDefault<UDisasterRecoverClientConfig>()->ImportedSessionMaxCount);
}

#undef LOCTEXT_NAMESPACE
