// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisasterRecoveryTasks.h"
#include "Misc/TVariant.h"

struct FFileChangeData;

/** Invoked when a new disaster recovery session is added to the list of sessions. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnDisasterRecoverySessionAdded, TSharedRef<FDisasterRecoverySession> /*Session*/);

/** Invoked when a new disaster recovery session is removed from the list of sessions. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnDisasterRecoverySessionRemoved, const FGuid& /*RepositoryId*/);

/** Invoked when a new disaster recovery session is updated. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnDisasterRecoverySessionUpdated, TSharedRef<FDisasterRecoverySession> /*Session*/);

class IConcertClientSession;
class FConcertActivityStream;
struct FConcertSessionActivity;

/**
 * Monitors, creates, restores, imports and inspects disaster recovery sessions.
 */
class FDisasterRecoverySessionManager
{
public:
	/**
	 * Construct the disaster recovery session manager.
	 * @param InRole The role as passed to Concert to identify the context in used.
	 * @param InSyncClient The disaster recovery sync client that will be used to connect to the server (Expected to be configured and started up)
	 * @note The manager resolves the Concert server corresponding the configured property UConcertClientConfig::DefaultServerURL.
	 */
	FDisasterRecoverySessionManager(const FString& InRole, TSharedPtr<IConcertSyncClient> InSyncClient);

	/** Destruct the disaster recovery session manager.*/
	~FDisasterRecoverySessionManager();

	/**
	 * Evaluates if at least one session is candidate to be recovered at boot time, so that the Recovery UI can be shown to the user. To be candidate, a
	 * session must not be mounted by another instance and must have at least one recoverable transactions.
	 * @return A future containing the result of the evaluation, true if a candidate exist, false otherwise.
	 * @note All evaluated candidates are locked by the process, preventing concurrent instances from loading/deleting the session.
	 */
	TFuture<bool> HasRecoverableCandidates();

	/**
	 * Get the latest known list of sessions currently tracked by the manager. The session list is subject to change when a session is created/deleted/updated by this process or a concurrent one.
	 * @return The current session list.
	 * @see OnSessionAdded(), OnSessionRemoved(), OnSessionUpdated().
	 */
	const TArray<TSharedRef<FDisasterRecoverySession>>& GetSessions() const { return SessionsCache; }

	/** Invoked when a session is added to the session list. */
	FOnDisasterRecoverySessionAdded& OnSessionAdded() { return OnDisasterRecoverySessionAddedDelegate; }

	/** Invoked when a session is removed from the session list. */
	FOnDisasterRecoverySessionRemoved& OnSessionRemoved() { return OnDisasterRecoverySessionRemovedDelegate; }

	/** Invoked when a session is updated. */
	FOnDisasterRecoverySessionUpdated& OnSessionUpdated() { return OnDisasterRecoverySessionUpdatedDelegate; }

	/**
	 * Load the specified session activities into an activity stream if the session repository is not already mounted by another process.
	 * @param Session The session to load.
	 * @return A future containing the activity stream if succeeded or an error message in case of failure.
	 */
	TFuture<TVariant<TSharedPtr<FConcertActivityStream>, FText>> LoadSession(TSharedRef<FDisasterRecoverySession> Session);

	/**
	 * Import a session from a crash report for inspection. The function expects to find the 'SessionInfo.json' and 'Session.db' files and optionally the 'Transactions' and 'Packages' directory
	 * in the same folder. The function will recreate a repository for the session and copy the files. The session activities is not loaded until the user call LoadSession().
	 * @param SessionInfoPathname Path to the 'SessionInfo.json'
	 * @return A pointer on the session if the session was imported or an error message in case of failure.
	 * @note OnSessionAdded() is invoked on successful import.
	 */
	TVariant<TSharedPtr<FDisasterRecoverySession>, FText> ImportSession(const FString& SessionInfoPathname);

	/**
	 * Discard the specified session, deleting all its files. The session cannot be discarded if it repository is mounted by another process.
	 * @param The session to discard.
	 * @return True if the session was discarded, false and an error message if it failed.
	 * @note OnSessionRemoved() is invoked if the session is discarded successfully.
	 */
	TFuture<TPair<bool, FText>> DiscardSession(const FDisasterRecoverySession& Session);

	/**
	 * Create a new session and join it.
	 * @return True if the session is created and joined successfully, false and an error message if an error occurred.
	 * @note OnSessionAdded() is invoked on successful creation.
	 */
	TFuture<TPair<bool, FText>> CreateAndJoinSession();

	/**
	 * Create a new session from an archived session, up to a given point in time in that session.
	 * @param ArchivedSession The archived session to restore.
	 * @param ThroughActivity Restore the archived session activities up to this point in time (included).
	 * @return True if the session was discarded, false and an error message if it failed.
	 * @note OnSessionAdded() is invoked if the session is created and the original archived session remains unmodified.
	 */
	TFuture<TPair<bool, FText>> RestoreAndJoinSession(TSharedPtr<FDisasterRecoverySession> ArchivedSession, TSharedPtr<FConcertSessionActivity> ThroughActivity);

	/** Leave the current live session, if any. If is safe to call the function even if the user doesn't have a live session in progress. */
	void LeaveSession();

	/** Check if the client has a live session in progress. This return true as soon as CreateAndJoinSession()/RestoreAndJoinSession() is called and returns false after LeaveSession() returns (or an error occurs provoking the user to not connect/disconnect). */
	bool HasInProgressSession() const;

	/** Reload the recovery info file, update the session cache and fire delegates accordingly. This detects if a concurrent instance crashed (without updating the itself) and update the state accordingly. */
	void Refresh();

private:
	/** Rotate and return the list of expired sessions. */
	TArray<FGuid> InitAndRotateSessions();

	/** Reload the recovery info file, update the cached sessions and notify the modifications. */
	void UpdateSessionsCache();

	/** Discard one or more session repositories. Discarding a repository is just like discarding the session because this manager always keeps one session per repository. */
	TFuture<TPair<bool, FText>> DiscardRepositories(TArray<FGuid> RepositoryIds);

	/** Invoked when a recovery session is created (from scratch or from a recovery). */
	void OnRecoverySessionStartup(TSharedRef<IConcertClientSession> InSession);

	/** Invoked when a recovery session is shut down normally. */
	void OnRecoverySessionShutdown(TSharedRef<IConcertClientSession> InSession);

	/** Invokey when a session was not found on the server. */
	void OnSessionNotFoundInternal(const FGuid& RepositoryId);

	/** Invoked when a set of repositories were successfully discarded by this process. */
	void OnSessionRepositoriesDiscardedInternal(const TArray<FGuid>& RepositoryIds);

	/** Invokey when a repository was successfully mounted by this process. */
	void OnSessionRepositoryMountedInternal(const FGuid& RepositoryId);

	/** Invoked when a session was imported (the expected files found and copied). */
	void OnSessionImportedInternal(const TSharedRef<FDisasterRecoverySession>& ImportedSession);

	/** Returns this client repository database root dir. */
	FString GetSessionRepositoryRootDir() const;

	/** Invoked when a file in the session repository directory is modified. Monitor changes made to the recovery info file by other processes. */
	void OnSessionRepositoryFilesModified(const TArray<FFileChangeData>& InFileChanges);

	/** Returns the path to the directory contining the recovery info file.*/
	FString GetDisasterRecoveryInfoPath() const;

	/** Returns the name of the recovery info file. */
	const TCHAR* GetDisasterRecoveryInfoFilename() const;

	/** Returns the pathnname the recovery info file. */
	FString GetDisasterRecoveryInfoPathname() const;

	/** Returns the number of 'recent' sessions to keep around for a given project. */
	int32 GetRecentSessionMaxCount() const;

	/** Returns the number of 'imported' sessions to keep around for a given project. */
	int32 GetImportedSessionMaxCount() const;

private:
	TSharedPtr<IConcertSyncClient> SyncClient;
	FString Role;
	FGuid ServerAdminEndpointId;
	TArray<TSharedRef<FDisasterRecoverySession>> SessionsCache;
	FOnDisasterRecoverySessionAdded OnDisasterRecoverySessionAddedDelegate;
	FOnDisasterRecoverySessionRemoved OnDisasterRecoverySessionRemovedDelegate;
	FOnDisasterRecoverySessionUpdated OnDisasterRecoverySessionUpdatedDelegate;

	FGuid CurrentSessionRepositoryId;
	FString CurrentSessionName;
	FDisasterRecoveryTaskExecutor TaskExecutor;
	int32 DisasterRecoveryClientCount = 0;

	FDelegateHandle DirectoryWatcherHandle;
	FString RecoveryInfoPathname;

	TArray<int32> CrashingProcessIds;
	uint32 SessionsCacheRevision = 0;
	bool bRefreshing = false;
};
