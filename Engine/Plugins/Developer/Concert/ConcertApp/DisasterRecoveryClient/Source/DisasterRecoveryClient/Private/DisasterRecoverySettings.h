// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisasterRecoverySettings.generated.h"

UCLASS(config=Engine)
class DISASTERRECOVERYCLIENT_API UDisasterRecoverClientConfig : public UObject
{
	GENERATED_BODY()
public:
	UDisasterRecoverClientConfig();

	/**
	 * Enables Recovery Hub plugin to create and/or restore a recovery sessions when needed.
	 */
	UPROPERTY(config, EditAnywhere, Category="Client Settings")
	bool bIsEnabled = true;

	/**
	 * The root directory where recovery sessions should be stored. If not set or
	 * invalid, the recovery sessions are stored in the project saved directory. The
	 * existing sessions are not moved (but remains accessible) when the directory is changed.
	 */
	UPROPERTY(config, EditAnywhere, Category="Client Settings")
	FDirectoryPath RecoverySessionDir;

	/**
	 * The maximum number of recent recovery sessions to keep around. The sessions are rotated
	 * at Editor startup and oldest ones are discarded.
	 */
	UPROPERTY(config, EditAnywhere, Category="Client Settings",  meta=(DisplayName="Session History Size", ClampMin = "0", UIMin = "0", UIMax = "50"))
	int32 RecentSessionMaxCount = 4;

	/**
	 * The maximum number of imported recovery session to keep around. The sessions are rotated
	 * at Editor startup and oldest imports are discarded.
	 */
	UPROPERTY(config, EditAnywhere, Category="Client Settings", meta=(DisplayName="Imported Session History Size", ClampMin = "0", UIMin = "0", UIMax = "50"))
	int32 ImportedSessionMaxCount = 4;
};
