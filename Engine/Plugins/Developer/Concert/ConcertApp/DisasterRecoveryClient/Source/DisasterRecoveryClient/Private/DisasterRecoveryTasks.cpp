// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisasterRecoveryTasks.h"
#include "ConcertMessages.h"
#include "IConcertClient.h"
#include "ConcertSyncSessionTypes.h"
#include "IConcertClientWorkspace.h"
#include "IConcertSyncClient.h"
#include "IDisasterRecoveryClientModule.h"
#include "ConcertActivityStream.h"
#include "DisasterRecoverySessionInfo.h"

#define LOCTEXT_NAMESPACE "DisasterRecoveryTasks"

namespace DisasterRecoveryTasksUtil
{

FText GetAbortMessage()
{
	return LOCTEXT("TaskAborted", "Task aborted.");
}

FText GetSystemErrorMessage(EConcertResponseCode Code)
{
	return LOCTEXT("SystemError", "System error.");
}

} // DisasterRecoveryTasksUtil


FDisasterRecoveryTaskExecutor::FDisasterRecoveryTaskExecutor()
{
	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FDisasterRecoveryTaskExecutor::Tick), 0.0f);
}

FDisasterRecoveryTaskExecutor::~FDisasterRecoveryTaskExecutor()
{
	FTSTicker::RemoveTicker(TickerHandle);
	Abort();
}

void FDisasterRecoveryTaskExecutor::Enqueue(TSharedPtr<FDisasterRecoveryTask> Task)
{
	Tasks.Add(MakeTuple(MoveTemp(Task), /*Started*/false));
}

bool FDisasterRecoveryTaskExecutor::Tick(float)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FDisasterRecoveryTaskExecutor_Tick);

	if (Tasks.Num())
	{
		if (!Tasks[0].Value) // Not started?
		{
			Tasks[0].Key->Start();
			Tasks[0].Value = true;
		}

		if (Tasks[0].Key->Tick()) // Done?
		{
			Tasks.RemoveAt(0);
		}
	}

	return true;
}

void FDisasterRecoveryTaskExecutor::Abort()
{
	for (TPair<TSharedPtr<FDisasterRecoveryTask>, bool>& Task : Tasks)
	{
		Task.Key->Abort();
	}
	Tasks.Reset();
}


FLookupDisasterRecoveryServerTask::FLookupDisasterRecoveryServerTask(IConcertClientRef InClient, FString InServerName, TFunction<void(const FGuid&)> InServerFoundFn, TFunction<void(const FText&)> InReportErrorFn)
	: Client(MoveTemp(InClient))
	, ServerName(MoveTemp(InServerName))
	, OnServerFoundFn(MoveTemp(InServerFoundFn))
	, OnErrorFn(MoveTemp(InReportErrorFn))
{
	Client->StartDiscovery();
}

FLookupDisasterRecoveryServerTask::~FLookupDisasterRecoveryServerTask()
{
	Client->StopDiscovery();
}

void FLookupDisasterRecoveryServerTask::Start()
{
	UE_LOGF(LogDisasterRecovery, Verbose, "Looking up disaster recovery server...");
}

bool FLookupDisasterRecoveryServerTask::Tick()
{
	TArray<FConcertServerInfo> Servers = Client->GetKnownServers();
	for (const FConcertServerInfo& ServerInfo : Servers)
	{
		if (ServerInfo.ServerName == ServerName)
		{
			OnServerFoundFn(ServerInfo.AdminEndpointId);
			UE_LOGF(LogDisasterRecovery, Verbose, "...Looking up disaster recovery server -> Found");
			return true; // Done
		}
	}

	return false; // Continue
}

void FLookupDisasterRecoveryServerTask::Abort()
{
	OnErrorFn(DisasterRecoveryTasksUtil::GetAbortMessage());
}


FDisasterRecoveryFutureExecTask::FDisasterRecoveryFutureExecTask(TFunction<void(const FText&)> InErrorOccurredFn)
	: OnErrorFn(MoveTemp(InErrorOccurredFn))
	, FutureContinuationToken(MakeShared<uint8>(0))
{
}

bool FDisasterRecoveryFutureExecTask::Tick()
{
	return bDone;
}

void FDisasterRecoveryFutureExecTask::Abort()
{
	if (!bDone)
	{
		OnErrorFn(DisasterRecoveryTasksUtil::GetAbortMessage());
		bDone = true;
	}
}


FMountDisasterRecoverySessionRepositoryTask::FMountDisasterRecoverySessionRepositoryTask(TFunction<TFuture<FConcertAdmin_MountSessionRepositoryResponse>()> InMountFn, TFunction<void()> InRepositoryMountedFn, TFunction<void(const FText&)> InErrorOccurredFn)
	: FDisasterRecoveryFutureExecTask(MoveTemp(InErrorOccurredFn))
	, MountFn(MoveTemp(InMountFn))
	, OnRepositoryMountedFn(InRepositoryMountedFn)
{
}

void FMountDisasterRecoverySessionRepositoryTask::Start()
{
	UE_LOGF(LogDisasterRecovery, Verbose, "Mounting session repository...");

	TWeakPtr<uint8> ExecutionToken = FutureContinuationToken;
	MountFn().Next([this, ExecutionToken](const FConcertAdmin_MountSessionRepositoryResponse& Response)
	{
		if (!ExecutionToken.IsValid()) // The task was deleted?
		{
			return;
		}
		else if (Response.ResponseCode != EConcertResponseCode::Success)
		{
			OnErrorFn(DisasterRecoveryTasksUtil::GetSystemErrorMessage(Response.ResponseCode));
		}
		else if (Response.MountStatus == EConcertSessionRepositoryMountResponseCode::NotFound)
		{
			OnErrorFn(LOCTEXT("RepositoryNotFound", "Repository not found."));
		}
		else if (Response.MountStatus == EConcertSessionRepositoryMountResponseCode::AlreadyMounted)
		{
			OnErrorFn(LOCTEXT("RepositoryAlreadyMounted", "Repository is locked by another process."));
		}
		else
		{
			check(Response.MountStatus == EConcertSessionRepositoryMountResponseCode::Mounted);
			OnRepositoryMountedFn();
		}
		
		UE_LOGF(LogDisasterRecovery, Verbose, "...Mounting session repository -> Succeeded");
		bDone = true; // Will be reported on next Tick().
	});
}


FDiscardDisasterRecoveryRepositoriesTask::FDiscardDisasterRecoveryRepositoriesTask(TFunction<TFuture<FConcertAdmin_DropSessionRepositoriesResponse>()> InDiscardRepositoryFn, TFunction<void(const TArray<FGuid>&)> InDiscardedRepositoriesFn, TFunction<void(const FText&)> InErrorOccurredFn)
	: FDisasterRecoveryFutureExecTask(MoveTemp(InErrorOccurredFn))
	, DropRepositoryFn(MoveTemp(InDiscardRepositoryFn))
	, OnRepositoriesDiscardedFn(MoveTemp(InDiscardedRepositoriesFn))
{
}

void FDiscardDisasterRecoveryRepositoriesTask::Start()
{
	UE_LOGF(LogDisasterRecovery, Verbose, "Discarding recovery sessions...");

	TWeakPtr<uint8> ExecutionToken = FutureContinuationToken;
	DropRepositoryFn().Next([this, ExecutionToken](const FConcertAdmin_DropSessionRepositoriesResponse& Response)
	{
		if (!ExecutionToken.IsValid()) // The task was deleted?
		{
			return;
		}
		else if (Response.ResponseCode != EConcertResponseCode::Success)
		{
			OnErrorFn(DisasterRecoveryTasksUtil::GetSystemErrorMessage(Response.ResponseCode));
			UE_LOGF(LogDisasterRecovery, Verbose, "...Discarding recovery sessions -> Failed");
		}
		else
		{
			OnRepositoriesDiscardedFn(Response.DroppedRepositoryIds);
			UE_LOGF(LogDisasterRecovery, Verbose, "...Discarding recovery sessions -> Succeeded");
		}

		bDone = true; // Will be reported on next Tick().
	});
}


FLookupDisasterRecoverySessionIdTask::FLookupDisasterRecoverySessionIdTask(TFunction<TFuture<FConcertAdmin_GetAllSessionsResponse>()> InGetSessionsFn, FString SessionName, TFunction<void(const FGuid&)> InResultFn, TFunction<void(const FText&)> InErrorOccurredFn)
	: FDisasterRecoveryFutureExecTask(MoveTemp(InErrorOccurredFn))
	, GetSessionsFn(MoveTemp(InGetSessionsFn))
	, OnResultFn(MoveTemp(InResultFn))
	, SessionName(MoveTemp(SessionName))
{
}

void FLookupDisasterRecoverySessionIdTask::Start()
{
	UE_LOGF(LogDisasterRecovery, Verbose, "Looking up recovery session '%ls'...", *SessionName);

	TWeakPtr<uint8> ExecutionToken = FutureContinuationToken;
	GetSessionsFn().Next([this, ExecutionToken](const FConcertAdmin_GetAllSessionsResponse& Response)
	{
		if (!ExecutionToken.IsValid()) // The task was deleted?
		{
			return;
		}
		else if (Response.ResponseCode != EConcertResponseCode::Success)
		{
			OnErrorFn(DisasterRecoveryTasksUtil::GetSystemErrorMessage(Response.ResponseCode));
			UE_LOGF(LogDisasterRecovery, Verbose, "...Looking up recovery session -> Failed");
		}
		else if (const FConcertSessionInfo* ArchivedSession = Response.ArchivedSessions.FindByPredicate([this](const FConcertSessionInfo& MatchCandidate){ return MatchCandidate.SessionName == SessionName; }))
		{
			OnResultFn(ArchivedSession->SessionId);
			UE_LOGF(LogDisasterRecovery, Verbose, "...Looking up recovery session -> Found archive session ID %ls", *ArchivedSession->SessionId.ToString());
		}
		else if (const FConcertSessionInfo* LiveSession = Response.LiveSessions.FindByPredicate([this](const FConcertSessionInfo& MatchCandidate){ return MatchCandidate.SessionName == SessionName; })) // The user live session?
		{
			OnResultFn(LiveSession->SessionId);
			UE_LOGF(LogDisasterRecovery, Verbose, "...Looking up recovery session -> Found live session ID %ls", *LiveSession->SessionId.ToString());
		}
		else
		{
			UE_LOGF(LogDisasterRecovery, Verbose, "...Looking up recovery session -> Not found");
			OnResultFn(FGuid()); // Not found -> Report an invalid Guid.
		}

		bDone = true; // Will be reported on next Tick().
	});
}


FCreateAndJoinDisasterRecoverySessionTask::FCreateAndJoinDisasterRecoverySessionTask(TFunction<TFuture<EConcertResponseCode>()> InCreateAndJoinFn, TSharedPtr<IConcertSyncClient> InSyncClient, FString InSessionName, TFunction<void()> InSessionJoined, TFunction<void(const FText&)> InErrorOccurredFn)
	: FDisasterRecoveryFutureExecTask(MoveTemp(InErrorOccurredFn))
	, CreateAndJoinFn(MoveTemp(InCreateAndJoinFn))
	, SyncClient(MoveTemp(InSyncClient))
	, SessionName(MoveTemp(InSessionName))
	, OnSessionJoinedFn(MoveTemp(InSessionJoined))
{
	SyncClient->GetConcertClient()->OnSessionConnectionChanged().AddRaw(this, &FCreateAndJoinDisasterRecoverySessionTask::OnSessionConnectionChanged);
}

FCreateAndJoinDisasterRecoverySessionTask::~FCreateAndJoinDisasterRecoverySessionTask()
{
	SyncClient->GetConcertClient()->OnSessionConnectionChanged().RemoveAll(this);
	if (TSharedPtr<IConcertClientWorkspace> Workspace = SyncClient->GetWorkspace())
	{
		Workspace->OnWorkspaceSynchronized().RemoveAll(this);
	}
}

void FCreateAndJoinDisasterRecoverySessionTask::Start()
{
	UE_LOGF(LogDisasterRecovery, Verbose, "Creating/Restoring session '%ls'...", *SessionName);

	TWeakPtr<uint8> ExecutionToken = FutureContinuationToken;
	CreateAndJoinFn().Next([this, ExecutionToken](EConcertResponseCode Response)
	{
		if (!ExecutionToken.IsValid()) // The task was deleted?
		{
			return;
		}
		else if (Response != EConcertResponseCode::Success)
		{
			if (!bDone) // Ensure the task is not already done from a cancellation.
			{
				OnErrorFn(DisasterRecoveryTasksUtil::GetSystemErrorMessage(Response));
				bDone = true;
				UE_LOGF(LogDisasterRecovery, Warning, "Failed to create or restore session %ls. Disaster recovery will not be enabled for this session.", *SessionName);
			}
		}
	});
}

bool FCreateAndJoinDisasterRecoverySessionTask::Tick()
{
	if (bWorkspaceSynchronized)
	{
		if (LoopCountSinceWorkspaceSync++ > 0) // Ensure FConcertClientWorkspace::OnEndFrame() is called once before saying the session is joined. It needs to run in case the session changes needs to be persisted.
		{
			OnSessionJoinedFn();
			bDone = true;
			UE_LOGF(LogDisasterRecovery, Verbose, "...Creating/Restoring session '%ls' -> Succeeded", *SessionName);
		}
	}
	return FDisasterRecoveryFutureExecTask::Tick();
}

void FCreateAndJoinDisasterRecoverySessionTask::OnSessionConnectionChanged(IConcertClientSession& Session, EConcertConnectionStatus Status)
{
	// Only handle the connection events corresponding to this task if concurrent tasks exist. (A task added just after another was aborted while in-flight)
	if (Session.GetSessionInfo().SessionName != SessionName)
	{
		return;
	}
	else if (Status == EConcertConnectionStatus::Connected)
	{
		SyncClient->GetWorkspace()->OnWorkspaceSynchronized().AddRaw(this, &FCreateAndJoinDisasterRecoverySessionTask::OnWorkspaceSynchronized);
	}
	else if (Status == EConcertConnectionStatus::Disconnected)
	{
		if (!bDone)
		{
			OnErrorFn(FText::Format(LOCTEXT("FailToJoinSession", "Failed to join the recovery session {0}."), FText::AsCultureInvariant(SessionName)));
			bDone = true;
			UE_LOGF(LogDisasterRecovery, Verbose, "...Creating/Restoring session '%ls' -> Failed (Disconnected)", *SessionName);
		}
	}
}

void FCreateAndJoinDisasterRecoverySessionTask::OnWorkspaceSynchronized()
{
	UE_LOGF(LogDisasterRecovery, Verbose, "Recovery session workspace synched");
	bWorkspaceSynchronized = true;
}


FFindRecoverableDisasterRecoverySessionTask::FFindRecoverableDisasterRecoverySessionTask(IConcertClientRef InClient, TFunction<FGuid()> InGetServerAdminEndpointId, TArray<TSharedPtr<FDisasterRecoverySession>> InCandidates, TFunction<void(TSharedPtr<FDisasterRecoverySession>)> OnCandidateSearchResult, TFunction<void(const FText&)> InErrorOccurredFn)
	: Client(MoveTemp(InClient))
	, GetServerAdminEndpointIdFn(MoveTemp(InGetServerAdminEndpointId))
	, Candidates(MoveTemp(InCandidates))
	, OnCandidateSearchResultFn(MoveTemp(OnCandidateSearchResult))
	, OnErrorFn(MoveTemp(InErrorOccurredFn))
	, ExecutionToken(MakeShared<uint8>(0)) // Execution token used to prevent future continuation execution in case this instance was deleted.
{
}

void FFindRecoverableDisasterRecoverySessionTask::Start()
{
	UE_LOGF(LogDisasterRecovery, Verbose, "Looking up a recoverable session...");
}

bool FFindRecoverableDisasterRecoverySessionTask::Tick()
{
	// Run a finite state machine to find a valid candidate to recover. (Must have activities and those activities must not be ignored on restore -> like activities performed during Multi-User session)

	if (!CurrentCandidate) // 1- The 'pick a candidate' state. Mount the next candidate in the list.
	{
		if (!Candidates.Num())
		{
			OnCandidateSearchResultFn(nullptr); // No candidate available. Transit to the 'exit' state.
			UE_LOGF(LogDisasterRecovery, Verbose, "...Looking up a recoverable session -> All candidates were eliminated");
			return true; // Done;
		}

		CurrentCandidate = Candidates.Last();
		UE_LOGF(LogDisasterRecovery, Verbose, "Evaluating candidate session '%ls'.", *CurrentCandidate->SessionName);

		TWeakPtr<uint8> Token = ExecutionToken; // Prevent future execution if 'this' was deleted.
		MountingTask = MakeUnique<FMountDisasterRecoverySessionRepositoryTask>(
			[this]() { return Client->MountSessionRepository(GetServerAdminEndpointIdFn(), CurrentCandidate->RepositoryRootDir, CurrentCandidate->RepositoryId, /*bCreateIfNotExist*/false, /*bAsDefault*/false); },
			[this, Token]() { if (Token.IsValid()) { bMounted = true; } },
			[this, Token](const FText&) { if (Token.IsValid()) { bMounted = false; } });

		MountingTask->Start(); // Transit to the 'mounting session repository' state.
	}
	else if (MountingTask) // 2- The 'mounting session repository' state.
	{
		if (MountingTask->Tick()) // Done?
		{
			MountingTask.Reset();
			if (bMounted)
			{
				TWeakPtr<uint8> Token = ExecutionToken; // Prevent future execution if 'this' was deleted.
				SearchingSessionIdTask = MakeUnique<FLookupDisasterRecoverySessionIdTask>(
					[this](){ return Client->GetServerSessions(GetServerAdminEndpointIdFn()); },
					Candidates.Last()->SessionName,
					[this, Token](const FGuid& ResultSessionId) { if (Token.IsValid()) { SessionId = ResultSessionId; } }, // Search result. The ID might be valid (found) or invalid (not found).
					[this, Token](const FText&) { if (Token.IsValid()) { SessionId.Invalidate(); } }); // Error == not found, report an invalid GUID.
				
				SearchingSessionIdTask->Start(); // Transit to the 'searching session' state.
			}
			else
			{
				UE_LOGF(LogDisasterRecovery, Verbose, "Eliminated candidate session '%ls'. Failed to mount the session repository.", *CurrentCandidate->SessionName);
				Candidates.Pop(); // Failed to mount -> likely mounted by another process, ignore this candidate.
				CurrentCandidate.Reset(); // Transit to 'Pick a candidate' state.
			}
		}
	}
	else if (SearchingSessionIdTask) // 3- The 'searching session' state.
	{
		if (SearchingSessionIdTask->Tick()) // Done?
		{
			SearchingSessionIdTask.Reset();
			if (SessionId.IsValid()) // Found?
			{
				ActivityStream = MakeShared<FConcertActivityStream>(Client, GetServerAdminEndpointIdFn(), SessionId, /*bInIncludeActivityDetails*/true); // Transit to 'load activities' state.
			}
			else
			{
				UE_LOGF(LogDisasterRecovery, Verbose, "Eliminated candidate session '%ls'. Failed to find the session files.", *CurrentCandidate->SessionName);
				Candidates.Pop(); // This candidate session could not be found (likely deleted).
				CurrentCandidate.Reset(); // Transit to 'pick a candidate' state.
			}
		}
	}
	else if (ActivityStream) // 4- The 'load activities' state.
	{
		int32 ReadCount;
		FText ErrorMsg;
		bool bEndOfStreamOrError = ActivityStream->Read(Activities, ReadCount, ErrorMsg); // Read activities (if any)

		if (ReadCount > 0) // On error, ReadCount is 0.
		{
			for (const TSharedPtr<FConcertSessionActivity>& Activity : Activities)
			{
				if (!Activity->Activity.bIgnored)
				{
					OnCandidateSearchResultFn(CurrentCandidate); // Found a candidate with recoverable activities, transit to the 'exit' state.
					UE_LOGF(LogDisasterRecovery, Verbose, "...Looking up a recoverable session -> Found recoverable session '%ls' among the candidates", *CurrentCandidate->SessionName);
					return true; // Found a recoverable candidate.
				}
			}
			Activities.Reset(); // Don't need to keep the activities.
		}

		if (bEndOfStreamOrError)
		{
			if (ErrorMsg.IsEmpty()) // No error.
			{
				UE_LOGF(LogDisasterRecovery, Verbose, "Eliminated candidate session %ls. No recoverable activities available.", *CurrentCandidate->SessionName);
			}
			else
			{
				UE_LOGF(LogDisasterRecovery, Warning, "Eliminated candidate session %ls. Reason: %ls", *CurrentCandidate->SessionName, *ErrorMsg.ToString());
			}

			Candidates.Pop();
			ActivityStream.Reset();
			CurrentCandidate.Reset();
		}
	}

	return false; // Tick again, not done.
}

void FFindRecoverableDisasterRecoverySessionTask::Abort()
{
	OnErrorFn(DisasterRecoveryTasksUtil::GetAbortMessage());
}

#undef LOCTEXT_NAMESPACE
