// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertClientModule.h"
#include "Containers/Ticker.h"
#include "Misc/Guid.h"

enum class EConcertConnectionStatus : uint8;
enum class EConcertResponseCode : uint8;
struct FConcertAdmin_DropSessionRepositoriesResponse;
struct FConcertAdmin_GetAllSessionsResponse;
struct FConcertAdmin_MountSessionRepositoryResponse;
template <typename ResultType> class TFuture;

class IConcertSyncClient;
class IConcertClientSession;
class FConcertActivityStream;
struct FConcertSessionActivity;
struct FDisasterRecoverySession;


/**
 * Represents a task the disaster recovery executor can run.
 */
class FDisasterRecoveryTask
{
public:
	virtual ~FDisasterRecoveryTask() = default;

	/** Invoked once by the task executor to starts the task. */
	virtual void Start() { }

	/** Invoked every frame by the task executor until the function returns true to signal this task completion. */
	virtual bool Tick() { return false; }

	/** Invoked by the task executor to abort the task. */
	virtual void Abort() = 0;
};

/**
 * Schedules and executes disaster recovery tasks serially. Using the task graph was ruled out because of the nature of
 * some tasks that needs to be ticked over many frames on the game thread.
 */
class FDisasterRecoveryTaskExecutor
{
public:
	/** Construct the executor. */
	FDisasterRecoveryTaskExecutor();

	/** Destruct the executor and abort all tasks. */
	~FDisasterRecoveryTaskExecutor();

	/** Schedule a task for execution, addind it at the end of the queue. The task will start only when previous tasks completed. */
	void Enqueue(TSharedPtr<FDisasterRecoveryTask> Task);

	/** Abort all executor tasks.*/
	void Abort();

private:
	/* Tick the executor tasks. */
	bool Tick(float);

private:
	TArray<TPair<TSharedPtr<FDisasterRecoveryTask>, bool /*bStarted*/>> Tasks;
	FTSTicker::FDelegateHandle TickerHandle;
};

/**
 * Lookup the specified recovery server.
 */
class FLookupDisasterRecoveryServerTask : public FDisasterRecoveryTask
{
public:
	FLookupDisasterRecoveryServerTask(IConcertClientRef InClient, FString InServerName, TFunction<void(const FGuid&)> InServerFoundFn, TFunction<void(const FText&)> InErrorOccurredFn); // TODO: Consider adding a timeout.
	virtual ~FLookupDisasterRecoveryServerTask();
	virtual void Start() override;
	virtual bool Tick() override;
	virtual void Abort() override;

private:
	IConcertClientRef Client;
	FString ServerName;
	TFunction<void(const FGuid&)> OnServerFoundFn;
	TFunction<void(const FText&)> OnErrorFn;
};

/**
 * Base task that start by executing a function returing a future and wait until the future returns. This class is expected to be
 * extended to override the Start() function.
 */
class FDisasterRecoveryFutureExecTask : public FDisasterRecoveryTask
{
public:
	FDisasterRecoveryFutureExecTask(TFunction<void(const FText&)> InErrorOccurredFn);
	virtual bool Tick() override;
	virtual void Abort() override;

protected:
	TFunction<void(const FText&)> OnErrorFn;
	TSharedPtr<uint8> FutureContinuationToken;
	bool bDone = false;
};

/**
 * Mount a session repository.
 */
class FMountDisasterRecoverySessionRepositoryTask : public FDisasterRecoveryFutureExecTask
{
public:
	FMountDisasterRecoverySessionRepositoryTask(TFunction<TFuture<FConcertAdmin_MountSessionRepositoryResponse>()> InMountFn, TFunction<void()> InRepositoryMountedFn, TFunction<void(const FText&)> InErrorOccurredFn);
	virtual void Start() override;

private:
	TFunction<TFuture<FConcertAdmin_MountSessionRepositoryResponse>()> MountFn;
	TFunction<void()> OnRepositoryMountedFn;
};

/**
 * Discards a set of session reporitories.
 */
class FDiscardDisasterRecoveryRepositoriesTask : public FDisasterRecoveryFutureExecTask
{
public:
	FDiscardDisasterRecoveryRepositoriesTask(TFunction<TFuture<FConcertAdmin_DropSessionRepositoriesResponse>()> InDropRepositoryFn, TFunction<void(const TArray<FGuid>&)> InRepositoriesDiscardedFn, TFunction<void(const FText&)> InErrorOccurredFn);
	virtual void Start() override;

private:
	TFunction<TFuture<FConcertAdmin_DropSessionRepositoriesResponse>()> DropRepositoryFn;
	TFunction<void(const TArray<FGuid>&)> OnRepositoriesDiscardedFn;
};

/**
 * Lookup a specific session ID by name.
 */
class FLookupDisasterRecoverySessionIdTask : public FDisasterRecoveryFutureExecTask
{
public:
	FLookupDisasterRecoverySessionIdTask(TFunction<TFuture<FConcertAdmin_GetAllSessionsResponse>()> InGetSessionsFn, FString SessionName, TFunction<void(const FGuid&)> OnResultFn, TFunction<void(const FText&)> InErrorOccurredFn);
	virtual void Start() override;

private:
	TFunction<TFuture<FConcertAdmin_GetAllSessionsResponse>()> GetSessionsFn;
	TFunction<void(const FGuid&)> OnResultFn;
	FString SessionName;
};

/**
 * Creates a new session (restoring from an archive or not) and join that session.
 */
class FCreateAndJoinDisasterRecoverySessionTask : public FDisasterRecoveryFutureExecTask
{
public:
	/**
	 * Constructs the task.
	 * @param InCreateAndJoinFn The opaque function invoked on Start() to create the session. Expected to be IConcertClient::CreateSession() or IConcertClient::RestoreSession().
	 * @param InSyncClient The disaster recovery sync client used to create/restore the session.
	 * @param InSessionName The session name to create.
	 * @param InOnSessionJoined Callback invoked when the session is created and joined.
	 * @param InOnErrorFn The callback invoked if an error occurred (with the corresponding message) during the execution of the task or if it was aborted.
	 */
	FCreateAndJoinDisasterRecoverySessionTask(TFunction<TFuture<EConcertResponseCode>()> InCreateAndJoinFn, TSharedPtr<IConcertSyncClient> InSyncClient, FString InSessionName, TFunction<void()> InOnSessionJoined, TFunction<void(const FText& /*Msg*/)> InOnErrorFn);
	virtual ~FCreateAndJoinDisasterRecoverySessionTask();
	virtual void Start() override;
	virtual bool Tick() override;

private:
	void OnSessionConnectionChanged(IConcertClientSession& Session, EConcertConnectionStatus Status);
	void OnWorkspaceSynchronized();

private:
	TFunction<TFuture<EConcertResponseCode>()> CreateAndJoinFn;
	TSharedPtr<IConcertSyncClient> SyncClient;
	FString SessionName;
	TFunction<void()> OnSessionJoinedFn;
	int32 LoopCountSinceWorkspaceSync = 0;
	bool bWorkspaceSynchronized = false;
};

/**
 * Evaluates the list of disaster recovery candidates to find a session that can be recovered. A candidate is found when the
 * its repository can be mounted, its archive loaded and if it contains at least one recoverable activity.
 */
class FFindRecoverableDisasterRecoverySessionTask : public FDisasterRecoveryTask
{
public:
	/**
	 * Constructs the task.
	 * @param InClient The disaster recovery Concert sync client.
	 * @param InGetServerAdminEndpointId A function returing the disaster recovery server admin endpoint ID.
	 * @param InCandidates The list of candidate sessions to evaluate.
	 * @param InOnResult The callback invoked when a candidate if found (parameter not null) or when all candidates were eliminated (parameter null).
	 * @param InOnErrorFn The callback invoked if an error occurred (with the corresponding message) during the execution of the task or if it was aborted.
	 */
	FFindRecoverableDisasterRecoverySessionTask(IConcertClientRef InClient, TFunction<FGuid()> InGetServerAdminEndpointId, TArray<TSharedPtr<FDisasterRecoverySession>> InCandidates, TFunction<void(TSharedPtr<FDisasterRecoverySession> /*NullOrCandidate*/)> InOnResult, TFunction<void(const FText& /*Msg*/)> InOnErrorFn);
	virtual void Start() override;
	virtual bool Tick() override;
	virtual void Abort() override;

private:
	IConcertClientRef Client;
	TFunction<FGuid()> GetServerAdminEndpointIdFn;
	TArray<TSharedPtr<FDisasterRecoverySession>> Candidates;
	TSharedPtr<FDisasterRecoverySession> CurrentCandidate;
	TFunction<void(TSharedPtr<FDisasterRecoverySession>)> OnCandidateSearchResultFn;
	TFunction<void(const FText&)> OnErrorFn;
	TSharedPtr<FConcertActivityStream> ActivityStream;
	TArray<TSharedPtr<FConcertSessionActivity>> Activities;
	TUniquePtr<FMountDisasterRecoverySessionRepositoryTask> MountingTask;
	TUniquePtr<FLookupDisasterRecoverySessionIdTask> SearchingSessionIdTask;
	FGuid SessionId;
	bool bMounted = false;
	TSharedPtr<uint8> ExecutionToken;
};
