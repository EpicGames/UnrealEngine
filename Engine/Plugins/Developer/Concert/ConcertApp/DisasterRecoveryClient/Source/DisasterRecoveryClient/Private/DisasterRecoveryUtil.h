// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace DisasterRecoveryUtil
{
	/** Returns the container name for the settings. */
	static inline FName GetSettingsContainerName() { return "Project"; }

	/** Returns the category name for the settings. */
	static inline FName GetSettingsCategoryName() { return "Plugins"; }

	/** Returns the section name for the settings. */
	static inline FName GetSettingsSectionName() { return "Disaster Recovery"; }
}
