// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisasterRecoveryHub.h"

#include "Async/Future.h"
#include "ConcertActivityStream.h"
#include "DisasterRecoverySessionInfo.h"
#include "DisasterRecoverySessionManager.h"
#include "DisasterRecoveryUtil.h"
#include "Framework/Views/TableViewMetadata.h"
#include "SConcertSessionRecovery.h"

#include "DesktopPlatformModule.h"
#include "EditorDirectories.h"
#include "EditorFontGlyphs.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "IDesktopPlatform.h"
#include "Misc/MessageDialog.h"
#include "ISettingsModule.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "SPositiveActionButton.h"
#include "SNegativeActionButton.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "SDisasterRecoveryHub"

namespace DisasterRecoveryUtil
{
	static const FName SessionColId = "Sessions";
}

/**
 * A node in the session tree view. It can represent a category (live, imported , etc) or a session.
 */
class FDisasterRecoverySessionTreeNode
{
public:
	FDisasterRecoverySessionTreeNode(FText InCategoryName) : CategoryName(MoveTemp(InCategoryName)) {}
	FDisasterRecoverySessionTreeNode(TSharedPtr<FDisasterRecoverySession> InSession) : Session(MoveTemp(InSession)) {}

	// Functions to use when the node represent a category (folder).
	bool IsCategoryNode() const { return Session == nullptr; }
	FText GetCategoryName() const { return CategoryName; }

	// Functions to use when the node represents a session (leaf).
	bool IsSessionNode() const { return Session != nullptr; }
	TSharedPtr<FDisasterRecoverySession> GetSession() { return Session; }
	const TArray<TSharedPtr<FDisasterRecoverySessionTreeNode>>& GetChildren() const { return Children; }
	void AddChild(TSharedPtr<FDisasterRecoverySessionTreeNode> Child) { Children.Add(Child); }
	bool RemoveChild(const FGuid& RepositoryId) { return Children.RemoveAll([&RepositoryId](const TSharedPtr<FDisasterRecoverySessionTreeNode>& Candidate) { return Candidate->GetSession()->RepositoryId == RepositoryId; }) > 0; }
	TSharedPtr<FDisasterRecoverySessionTreeNode> FindChild(const FDisasterRecoverySession& DesiredChild) const
	{
		TSharedPtr<FDisasterRecoverySessionTreeNode> Child;
		if (const TSharedPtr<FDisasterRecoverySessionTreeNode>* Node = Children.FindByPredicate([&DesiredChild](const TSharedPtr<FDisasterRecoverySessionTreeNode>& MatchCandidate) { return DesiredChild.RepositoryId == MatchCandidate->GetSession()->RepositoryId; }))
		{
			Child = *Node;
		}
		return Child;
	}

	bool IsLockedByAnotherProcess() const
	{
		return Session ? Session->MountedByProcessId != 0 && Session->MountedByProcessId != FPlatformProcess::GetCurrentProcessId() : false;
	}

private:
	TSharedPtr<FDisasterRecoverySession> Session;
	FText CategoryName;
	TArray<TSharedPtr<FDisasterRecoverySessionTreeNode>> Children;
};


/**
 * The widget displaying a session tree node.
 */
class SDisasterRecoverySessionTreeNodeWidget : public STableRow<TSharedPtr<FDisasterRecoverySessionTreeNode>>
{
public:
	typedef TFunction<void(TSharedPtr<FDisasterRecoverySessionTreeNode>)> FDoubleClickFunc;

	SLATE_BEGIN_ARGS(SDisasterRecoverySessionTreeNodeWidget)
		: _Font(FAppStyle::Get().GetFontStyle(TEXT("NormalFont")))
		{
		}
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)

	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, TSharedPtr<FDisasterRecoverySessionTreeNode> InTreeNode, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		TreeNode = MoveTemp(InTreeNode);

		TSharedPtr<SWidget> NodeWidget;
		if (TreeNode->IsSessionNode())
		{
			FText DisplayedSessionName;
			FString ProjectName;
			FDateTime DateTime;
			if (!RecoveryService::TokenizeSessionName(TreeNode->GetSession()->SessionName, nullptr, nullptr, &ProjectName, &DateTime))
			{
				// The session name format did not match the expect one. Likely a session name from 4.24 with a different format that was migrated to 4.25 as it.
				DisplayedSessionName = FText::AsCultureInvariant(TreeNode->GetSession()->SessionName);
			}
			else if (TreeNode->GetSession()->IsImported())
			{
				// Imported sessions (for crash inspection) may be from a different projects than the one currently opened, ensure to display the project name.
				DisplayedSessionName = FText::Format(LOCTEXT("DisplayedSessionName", "{0} {1}"), FText::AsCultureInvariant(ProjectName), FText::AsDateTime(DateTime));
			}
			else
			{
				// For this project session, just display the date.
				DisplayedSessionName = FText::AsDateTime(DateTime);
			}

			FSlateColor TextColor = TreeNode->GetSession()->IsUnreviewedCrash() ? FLinearColor(0.5f, 0.0f, 0.0f) : FSlateColor::UseForeground(); // Put a special color for unreviewed crashes.

			NodeWidget = SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(FText::AsCultureInvariant(DisplayedSessionName))
					.ColorAndOpacity(TextColor)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5, 0, 0, 0)
				[
					SNew(STextBlock)
					.Text(this, &SDisasterRecoverySessionTreeNodeWidget::GetLockedByProcessText)
					.Visibility(this, &SDisasterRecoverySessionTreeNodeWidget::GetLockedByProcessVisibility)
					.ColorAndOpacity(TextColor)
				];
		}
		else // Category node.
		{
			NodeWidget = SNew(STextBlock)
				.Text(TreeNode->GetCategoryName())
				.Font(InArgs._Font);
		}

		STableRow<TSharedPtr<FDisasterRecoverySessionTreeNode>>::Construct(
			STableRow<TSharedPtr<FDisasterRecoverySessionTreeNode>>::FArguments()
			.Content()
			[
				NodeWidget.ToSharedRef()
			],
			InOwnerTableView);
	}

	FText GetLockedByProcessText() const
	{
		return FText::Format(LOCTEXT("LockedByProcess", "(Locked by process {0})"), TreeNode->GetSession()->MountedByProcessId);
	}

	EVisibility GetLockedByProcessVisibility() const
	{
		if (TreeNode->IsLockedByAnotherProcess())
		{
			return EVisibility::Visible;
		}
		return EVisibility::Collapsed;
	}

private:
	TSharedPtr<FDisasterRecoverySessionTreeNode> TreeNode;
};


void SDisasterRecoveryHub::Construct(const FArguments& InArgs, const TSharedPtr<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow, TSharedPtr<FDisasterRecoverySessionManager> InSessionManager)
{
	SessionManager = MoveTemp(InSessionManager);
	OwnerTab = ConstructUnderMajorTab;
	OwnerWindow = ConstructUnderWindow;
	NextRefreshTime = FDateTime::UtcNow() + FTimespan(0, 0, 2); // Every 2 seconds.

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		.BorderBackgroundColor(FSlateColor(FLinearColor(0.6f, 0.6f, 0.6f)))
		.Padding(0)
		[
			SNew(SVerticalBox)

			// Provide a 'context' to the user why this UI is shown.
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			.Padding(0, 6)
			[
				SNew(STextBlock)
				.Text(InArgs._IntroductionText)
				.Visibility(InArgs._IntroductionText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible)
			]

			// The toolbar containing the buttons.
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				.Padding(FMargin(2.0f, 2.0f))
				[
					MakeToolbarWidget()
				]
			]

			+SVerticalBox::Slot()
			[
				SNew(SSplitter)
				.Orientation(EOrientation::Orient_Horizontal)

				+SSplitter::Slot() // Display the session lists.
				.Value(0.25)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
					.Padding(FMargin(2.0f, 2.0f))
					[
						MakeSessionTreeView()
					]
				]

				+SSplitter::Slot() // Session activities
				[
					SNew(SSplitter)
					.Orientation(EOrientation::Orient_Vertical)

					+SSplitter::Slot() // Session activities.
					[
						MakeSessionActivityView()
					]
				]
			]

			// RecoveryAll/Cancel buttons
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(0, 6)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FMargin(2.0f, 0.0f))
				.Visibility(InArgs._IsRecoveryMode ? EVisibility::Visible : EVisibility::Collapsed)

				+SUniformGridPanel::Slot(0, 0)
				[
					SNew(SPositiveActionButton)
					.Icon(FAppStyle::Get().GetBrush("Icons.Refresh"))
					.ToolTipText(this, &SDisasterRecoveryHub::GetRecoverAllButtonTooltip)
					.OnClicked(this, &SDisasterRecoveryHub::OnRecoverAllButtonClicked)
					.IsEnabled(this, &SDisasterRecoveryHub::IsRecoverAllButtonEnabled)
					.Text(LOCTEXT("RecoverAll", "Recover All"))
				]

				+SUniformGridPanel::Slot(1, 0)
				[
					SNew(SNegativeActionButton)
					.Icon(FAppStyle::Get().GetBrush("Icons.Delete"))
					.ToolTipText(LOCTEXT("CancelRecoveryTooltip", "Discard any recoverable data for your assets and continue with their last saved state"))
					.OnClicked(this, &SDisasterRecoveryHub::OnCancelButtonClicked)
					.Text(LOCTEXT("Discard", "Discard"))
				]
			]
		]
	];

	// Create the category node displayed in the tree view. Don't add them to the tree view until they have at least 1 child.
	UnreviewedCrashCategoryRootNode = MakeShared<FDisasterRecoverySessionTreeNode>(LOCTEXT("UnreviewedCrashCategory", "Unreviewed Crashes"));
	LiveCategoryRootNode = MakeShared<FDisasterRecoverySessionTreeNode>(LOCTEXT("LiveSessionsCategory", "Live Sessions"));
	RecentCategoryRootNode = MakeShared<FDisasterRecoverySessionTreeNode>(LOCTEXT("RecentSessionsCategory", "Recents Sessions"));
	ImportedCategoryRootNode = MakeShared<FDisasterRecoverySessionTreeNode>(LOCTEXT("ImportedSessionCategory", "Imported Sessions"));

	// Add the sessions.
	for (TSharedRef<FDisasterRecoverySession> Session : SessionManager->GetSessions())
	{
		OnSessionAdded(MoveTemp(Session));
	}

	// Listen for changes to the sessions (if another Editor instance runs or starts)
	SessionManager->OnSessionAdded().AddSP(this, &SDisasterRecoveryHub::OnSessionAdded);
	SessionManager->OnSessionUpdated().AddSP(this, &SDisasterRecoveryHub::OnSessionUpdated);
	SessionManager->OnSessionRemoved().AddSP(this, &SDisasterRecoveryHub::OnSessionRemoved);

	// If a 'pending review' exist, select it by default (if many exist, arbitrary select the first one, the user will be able to view them all)
	if (InArgs._IsRecoveryMode && UnreviewedCrashCategoryRootNode->GetChildren().Num())
	{
		SessionTreeView->SetSelection(UnreviewedCrashCategoryRootNode->GetChildren()[0]);
	}

	SessionTreeView->RequestTreeRefresh();
}

TSharedRef<SWidget> SDisasterRecoveryHub::MakeToolbarWidget()
{
	FName IconFontStyleName(TEXT("FontAwesome.16"));

	return SNew(SHorizontalBox)

		+SHorizontalBox::Slot() // Import button
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
			.OnClicked(this, &SDisasterRecoveryHub::OnImportClicked)
			.ToolTipText(LOCTEXT("ImportTooltip", "Import a crashed session for inspection."))
			.ContentPadding(FMargin(6, 4))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle(IconFontStyleName))
				.Text(FEditorFontGlyphs::Download)
			]
		]

		+SHorizontalBox::Slot() // Delete button
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
			.OnClicked(this, &SDisasterRecoveryHub::OnDeleteClicked)
			.IsEnabled(this, &SDisasterRecoveryHub::IsDeleteButtonEnabled)
			.ToolTipText(LOCTEXT("DeleteTooltip", "Delete the selected session."))
			.ContentPadding(FMargin(6, 4))
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle(IconFontStyleName))
				.Text(FEditorFontGlyphs::Trash)
			]
		]

		+SHorizontalBox::Slot() // Fill up the space.
		.FillWidth(1.0)
		[
			SNew(SSpacer)
		]

		+SHorizontalBox::Slot() // Config button
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
			.OnClicked(this, &SDisasterRecoveryHub::OnConfigClicked)
			.ToolTipText(LOCTEXT("ConfigureTooltip", "Open the configuration tab."))
			.ContentPadding(FMargin(6, 4))
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle(IconFontStyleName))
				.Text(FEditorFontGlyphs::Cogs)
			]
		];
}

FReply SDisasterRecoveryHub::OnImportClicked()
{
	static const FString SessionInfoFilename = TEXT("SessionInfo.json");

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);
	FString FileTypes = "SessionInfo.json|*.json"; // SessionInfo.json
	TArray<FString> OpenFilenames;
	int32 FilterIndex;

	bool bOpened = DesktopPlatform->OpenFileDialog(
		ParentWindowWindowHandle,
		LOCTEXT("ImportDialogTitle", "Import a Session").ToString(),
		FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_IMPORT), // Default path.
		SessionInfoFilename, // Default file
		FileTypes, // Filter
		EFileDialogFlags::None,
		OpenFilenames,
		FilterIndex);

	if (bOpened)
	{
		TVariant<TSharedPtr<FDisasterRecoverySession>, FText> ImportResult = SessionManager->ImportSession(OpenFilenames[0]);
		if (ImportResult.IsType<TSharedPtr<FDisasterRecoverySession>>()) // Was imported.
		{
			// On successful import, OnSessionAdded() was called and the node added under the 'imported' category.
			if (TSharedPtr<FDisasterRecoverySessionTreeNode> Node = ImportedCategoryRootNode->FindChild(*ImportResult.Get<TSharedPtr<FDisasterRecoverySession>>()))
			{
				SessionTreeView->SetSelection(Node);
			}
		}
		else // Show an error message.
		{
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("FailedToImportMsg", "Failed to import the session. {0}"), ImportResult.Get<FText>()), LOCTEXT("FailedToImportTitle", "Import Failed"));
		}
	}

	return FReply::Handled();
}

FReply SDisasterRecoveryHub::OnDeleteClicked()
{
	if (SelectedSessionNode && SelectedSessionNode->IsSessionNode() && !SelectedSessionNode->GetSession()->IsLive())
	{
		SessionManager->DiscardSession(*SelectedSessionNode->GetSession()); // TODO next phase: Consider adding a toast in case of failure (unlikely, but possible)
	}
	return FReply::Handled();
}

bool SDisasterRecoveryHub::IsDeleteButtonEnabled() const
{
	return SelectedSessionNode && SelectedSessionNode->IsSessionNode() && !SelectedSessionNode->GetSession()->IsLive() && !SelectedSessionNode->IsLockedByAnotherProcess();
}

FReply SDisasterRecoveryHub::OnConfigClicked()
{
	FModuleManager::GetModulePtr<ISettingsModule>("Settings")->ShowViewer(DisasterRecoveryUtil::GetSettingsContainerName(), DisasterRecoveryUtil::GetSettingsCategoryName(), DisasterRecoveryUtil::GetSettingsSectionName());
	return FReply::Handled();
}

TSharedRef<SWidget> SDisasterRecoveryHub::MakeSessionTreeView()
{
	return SAssignNew(SessionTreeView, STreeView<TSharedPtr<FDisasterRecoverySessionTreeNode>>)
	.TreeItemsSource(&SessionTreeRootNodes)
	.OnGenerateRow(this, &SDisasterRecoveryHub::OnGenerateSessionTreeNodeWidget)
	.OnGetChildren(this, &SDisasterRecoveryHub::OnGetSessionTreeNodeChildren)
	.OnSelectionChanged(this, &SDisasterRecoveryHub::OnSessionSelectionChanged)
	.OnSetExpansionRecursive(this, &SDisasterRecoveryHub::OnSetSessionTreeExpansionRecursive)
	.SelectionMode(ESelectionMode::Single)
	.HeaderRow(SNew(SHeaderRow)
		+SHeaderRow::Column(DisasterRecoveryUtil::SessionColId)
		.DefaultLabel(LOCTEXT("SessionColumnHeader", "Sessions")));
}

TSharedRef<ITableRow> SDisasterRecoveryHub::OnGenerateSessionTreeNodeWidget(TSharedPtr<FDisasterRecoverySessionTreeNode> TreeNode, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SDisasterRecoverySessionTreeNodeWidget, TreeNode, OwnerTable)
		.Font(TreeNode == UnreviewedCrashCategoryRootNode ? FAppStyle::Get().GetFontStyle(TEXT("BoldFont")) : FAppStyle::Get().GetFontStyle( TEXT("NormalFont")));
}

void SDisasterRecoveryHub::OnGetSessionTreeNodeChildren(TSharedPtr<FDisasterRecoverySessionTreeNode> InParent, TArray<TSharedPtr<FDisasterRecoverySessionTreeNode>>& OutChildren) const
{
	if (InParent->IsCategoryNode())
	{
		for (TSharedPtr<FDisasterRecoverySessionTreeNode> Child : InParent->GetChildren())
		{
			OutChildren.Add(MoveTemp(Child));
		}
	}
}

void SDisasterRecoveryHub::OnSetSessionTreeExpansionRecursive(TSharedPtr<FDisasterRecoverySessionTreeNode> InTreeNode, bool bInIsItemExpanded)
{
	if (InTreeNode.IsValid())
	{
		SessionTreeView->SetItemExpansion(InTreeNode, bInIsItemExpanded);
	}
}

void SDisasterRecoveryHub::OnSessionAdded(TSharedRef<FDisasterRecoverySession> InAddedSession)
{
	// Utility function to add a session node under a category node.
	auto AddSessionNodeFn = [this](TSharedRef<FDisasterRecoverySession> Session, TSharedPtr<FDisasterRecoverySessionTreeNode> CategoryNode, int32 CategoryNodeInsertPosition)
	{
		CategoryNode->AddChild(MakeShared<FDisasterRecoverySessionTreeNode>(MoveTemp(Session)));
		if (CategoryNode->GetChildren().Num() == 1) // Should the category node added to the tree?
		{
			SessionTreeRootNodes.Insert(CategoryNode, CategoryNodeInsertPosition);
			OnSetSessionTreeExpansionRecursive(CategoryNode, /*bInIsItemExpanded*/true);
		}
	};

	// The desired tree view root node order from top down is: 'Unreviewed Crashes', 'Live', 'Recents' then 'Imported.
	if (InAddedSession->IsRecent())
	{
		AddSessionNodeFn(MoveTemp(InAddedSession), RecentCategoryRootNode, ImportedCategoryRootNode->GetChildren().Num() == 0 ? SessionTreeRootNodes.Num() : SessionTreeRootNodes.Num() - 1); // The 'recent sessions' category goes before 'imported' category (if displayed).
	}
	else if (InAddedSession->IsImported())
	{
		AddSessionNodeFn(MoveTemp(InAddedSession), ImportedCategoryRootNode, SessionTreeRootNodes.Num()); // The 'imported sessions' category is always last.
	}
	else if (InAddedSession->IsUnreviewedCrash())
	{
		AddSessionNodeFn(MoveTemp(InAddedSession), UnreviewedCrashCategoryRootNode, 0); // The 'unreviewed crashes' category is always first.
	}
	else
	{
		check(InAddedSession->IsLive());
		AddSessionNodeFn(MoveTemp(InAddedSession), LiveCategoryRootNode, UnreviewedCrashCategoryRootNode->GetChildren().Num() ? 1 : 0); // The 'live sessions' category goes after the 'unreviewed crash' category (if displayed).
	}

	SessionTreeView->RequestTreeRefresh();
}

void SDisasterRecoveryHub::OnSessionUpdated(TSharedRef<FDisasterRecoverySession> InUpdatedSession)
{
	if (const TSharedPtr<FDisasterRecoverySessionTreeNode>* UnreviewedCrash = UnreviewedCrashCategoryRootNode->GetChildren().FindByPredicate([InUpdatedSession](const TSharedPtr<FDisasterRecoverySessionTreeNode>& MatchCandidate) { return MatchCandidate->GetSession()->RepositoryId == InUpdatedSession->RepositoryId; }))
	{
		if (InUpdatedSession->IsRecent()) // Was changed from 'unreviewed crash' to 'recent' -> The session was restored or abandonned by the user.
		{
			TSharedPtr<FDisasterRecoverySessionTreeNode> ToMove = *UnreviewedCrash;
			OnSessionRemoved(InUpdatedSession->RepositoryId);
			OnSessionAdded(InUpdatedSession);
		}
	}
	else if (const TSharedPtr<FDisasterRecoverySessionTreeNode>* Live = LiveCategoryRootNode->GetChildren().FindByPredicate([InUpdatedSession](const TSharedPtr<FDisasterRecoverySessionTreeNode>& MatchCandidate) { return MatchCandidate->GetSession()->RepositoryId == InUpdatedSession->RepositoryId; }))
	{
		if (InUpdatedSession->IsRecent()) // Was changed from 'live' to 'recent' -> The session ended normally and was moved to 'recent'.
		{
			TSharedPtr<FDisasterRecoverySessionTreeNode> ToMove = *Live;
			OnSessionRemoved(InUpdatedSession->RepositoryId);
			OnSessionAdded(InUpdatedSession);
		}
	}
	// else -> Other category changes are not expected. Few properties changes (like mounted by process ID) are expected to be handled using TAttribute<> to update in real time.
}

void SDisasterRecoveryHub::OnSessionRemoved(const FGuid& RemovedSessionRepositoryId)
{
	auto RemoveNodeFn = [this](TSharedPtr<FDisasterRecoverySessionTreeNode>& CategoryNode, const FGuid& SessionRepositoryId)
	{
		if (CategoryNode->RemoveChild(SessionRepositoryId))
		{
			if (CategoryNode->GetChildren().Num() == 0)
			{
				SessionTreeRootNodes.Remove(CategoryNode); // Don't keep empty root node in the tree view.
			}
			return true;
		}
		return false;
	};

	// The removed session can only be in one category.
	if (RemoveNodeFn(UnreviewedCrashCategoryRootNode, RemovedSessionRepositoryId) ||
		RemoveNodeFn(LiveCategoryRootNode, RemovedSessionRepositoryId) ||
		RemoveNodeFn(RecentCategoryRootNode, RemovedSessionRepositoryId) ||
		RemoveNodeFn(ImportedCategoryRootNode, RemovedSessionRepositoryId))
	{
		SessionTreeView->RequestTreeRefresh();
	}
}

void SDisasterRecoveryHub::OnSessionSelectionChanged(TSharedPtr<FDisasterRecoverySessionTreeNode> InSelectedNode, ESelectInfo::Type SelectInfo)
{
	SelectedSessionNode = InSelectedNode;
	SelectedSessionActivityStream.Reset();
	ActivityView->Reset();
	LoadingSessionErrorMsg = FText::GetEmpty();

	if (!InSelectedNode) // Selection was cleared.
	{
		return;
	}
	else if (TSharedPtr<FDisasterRecoverySession> RecoverySession = SelectedSessionNode->GetSession())
	{
		bLoadingActivities = true; // This enable displaying 'Loading Activities' in UI.
		TWeakPtr<FDisasterRecoverySessionTreeNode> WeakSelectedSession = SelectedSessionNode;
		SessionManager->LoadSession(RecoverySession.ToSharedRef()).Next([this, WeakSelectedSession](TVariant<TSharedPtr<FConcertActivityStream>, FText> Result)
		{
			TSharedPtr<FDisasterRecoverySessionTreeNode> SelectedNode = WeakSelectedSession.Pin();
			if (SelectedNode != nullptr && SelectedNode == SelectedSessionNode) // Selected session did not change since the time the call was made.
			{
				if (Result.IsType<TSharedPtr<FConcertActivityStream>>()) // Activity stream was created and user can scroll it.
				{
					SelectedSessionActivityStream = Result.Get<TSharedPtr<FConcertActivityStream>>();
				}
				else // The function failed.
				{
					bLoadingActivities = false;
					LoadingSessionErrorMsg = Result.Get<FText>();
				}
			}
		});
	}
}

TSharedRef<SWidget> SDisasterRecoveryHub::MakeSessionActivityView()
{
	return SAssignNew(ActivityView, SConcertSessionRecovery)
		.OnFetchActivities(this, &SDisasterRecoveryHub::FetchActivities)
		.WithClientAvatarColorColumn(false) // Disaster recovery has only one user, the local one.
		.WithClientNameColumn(false)
		.WithOperationColumn(true)
		.WithPackageColumn(true)
		.DetailsAreaVisibility(ShouldDisplayActivityDetails() ? EVisibility::Visible : EVisibility::Collapsed)
		.IsConnectionActivityFilteringEnabled(false) // Not valuable and ignored by Disaster Recovery (DR). DR sessions are for the local user only and join/leave are not recoverable but this will also ignore when the local user joins/leaves a Multi-User session.
		.IsLockActivityFilteringEnabled(false)       // Not valuable and ignored by Disaster Recovery (DR). DR sessions are for the local user only but lock/unlock are not recoverable but this will also ignore when the local user locks/unlocks assets in a Multi-User session.
		.IsPackageActivityFilteringEnabled(true)     // Enabled and displayed by default.
		.IsTransactionActivityFilteringEnabled(true) // Enabled and displayed by default.
		.IsReplicationActivityFilteringEnabled(true) // Enabled and displayed by default.
		.IsIgnoredActivityFilteringEnabled(true)     // Events ignored when restoring are not displayed by default. Enabled to inspect Multi-User transaction/package activities recorded (but not recoverable) by disaster recovery session in case the crash occurred during the Multi-User session.
		.AreRecoverAllAndCancelButtonsVisible(false) // Replaced by this widget own (for better placement)
		.IsRecoverThroughButtonsVisible(this, &SDisasterRecoveryHub::IsRecoverThroughButtonVisible)
		.OnRestore([this](TSharedPtr<FConcertSessionActivity> ThroughActivity) { OnRecoverThroughButtonClicked(ThroughActivity); return /*DismissWindow*/false; }) // Invoked when the 'Recover Through' button is clicked.
		.NoActivitiesReasonText(this, &SDisasterRecoveryHub::GetNoActivityDisplayedReason);
}

FText SDisasterRecoveryHub::GetNoActivityDisplayedReason() const
{
	if (ActivityView->GetTotalActivityNum() > 0)
	{
		return FText::GetEmpty(); // When no reason is provided, the text widget is collapsed.
	}
	else if (!SelectedSessionNode)
	{
		return LOCTEXT("NoSessionSelected", "No session selected");
	}
	else if (bLoadingActivities)
	{
		return LOCTEXT("LoadingActivities", "Loading Activities...");
	}
	else if (!LoadingSessionErrorMsg.IsEmpty())
	{
		return LoadingSessionErrorMsg;
	}
	else
	{
		return LOCTEXT("NoSessionActivities", "The selected session doesn't have any activities");
	}
}

bool SDisasterRecoveryHub::FetchActivities(TArray<TSharedPtr<FConcertSessionActivity>>& InOutActivities, int32& OutFetchedCount, FText& ErrorMsg)
{
	if (SelectedSessionActivityStream)
	{
		bool bEndOfStream = SelectedSessionActivityStream->Read(InOutActivities, OutFetchedCount, ErrorMsg);
		bLoadingActivities = !bEndOfStream;
		return bEndOfStream; // Done?
	}

	return false; // Not done until the stream is fully consumed.
}

bool SDisasterRecoveryHub::IsRecoverAllButtonEnabled() const
{
	if (!SelectedSessionNode)
	{
		return false;
	}
	else if (TSharedPtr<FDisasterRecoverySession> Session = SelectedSessionNode->GetSession())
	{
		return (Session->IsUnreviewedCrash()) && Session->MountedByProcessId == FPlatformProcess::GetCurrentProcessId();
	}
	return false;
}

FText SDisasterRecoveryHub::GetRecoverAllButtonTooltip() const
{
	return ActivityView->GetRecoverAllButtonTooltip();
}

FReply SDisasterRecoveryHub::OnRecoverAllButtonClicked()
{
	return RecoverThrough(ActivityView->GetMostRecentActivity());
}

bool SDisasterRecoveryHub::IsRecoverThroughButtonVisible() const
{
	return IsRecoverAllButtonEnabled();
}

void SDisasterRecoveryHub::OnRecoverThroughButtonClicked(TSharedPtr<FConcertSessionActivity> ThroughActivity)
{
	RecoverThrough(ThroughActivity);
}

FReply SDisasterRecoveryHub::RecoverThrough(TSharedPtr<FConcertSessionActivity> ThroughActivity)
{
	if (SelectedSessionNode && !SelectedSessionNode->IsCategoryNode() && ThroughActivity)
	{
		SessionManager->LeaveSession();
		SessionManager->RestoreAndJoinSession(SelectedSessionNode->GetSession(), ThroughActivity); // Note: Concert already provides the toast notifications on success/failure.
	}

	return DismissWidget();
}

FReply SDisasterRecoveryHub::OnCancelButtonClicked()
{
	return DismissWidget();
}

FReply SDisasterRecoveryHub::DismissWidget()
{
	if (TSharedPtr<SDockTab> OwnerTabPin = OwnerTab.Pin())
	{
		OwnerTabPin->RequestCloseTab();
	}
	else if (TSharedPtr<SWindow> OwnerWindowPin = OwnerWindow.Pin())
	{
		OwnerWindowPin->RequestDestroyWindow();
	}
	return FReply::Handled();
}

void SDisasterRecoveryHub::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (NextRefreshTime < FDateTime::UtcNow())
	{
		SessionManager->Refresh(); // Detect if a concurrent instance crashed and update the UI accordingly.
		NextRefreshTime = FDateTime::UtcNow() + FTimespan(0, 0, 2); // Every 2 seconds.
	}
}

#undef LOCTEXT_NAMESPACE
