// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class ITableRow;
class SDockTab;
class STableViewBase;
namespace ESelectInfo { enum Type : int; }
struct FDisasterRecoverySession;
template <typename ItemType> class STreeView;

class FConcertActivityStream;
class FDisasterRecoverySessionManager;
class SConcertSessionRecovery;
struct FConcertSessionActivity;
class FDisasterRecoverySessionTreeNode;
class SWindow;

/**
 * Displays, imports, inspects and restores disaster recovery sessions.
 */
class SDisasterRecoveryHub : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDisasterRecoveryHub)
		: _IsRecoveryMode(false)
		{
		}
		/** A short text explaining why this UI is presented to the user. */
		SLATE_ARGUMENT(FText, IntroductionText)

		/** Set the UI to run in recovery mode, i.e. that the widget is automatically displayed at boot time to recover from a previous crash rather than opened from the menu for inspection. */
		SLATE_ARGUMENT(bool, IsRecoveryMode)
	SLATE_END_ARGS();

	/**
	* Constructs the recovery hup.
	*
	* @param InArgs The Slate argument list.
	* @param ConstructUnderMajorTab The major tab which will contain the session front-end.
	* @param ConstructUnderWindow The window in which this widget is being constructed.
	* @param InSyncClient The sync client.
	*/
	void Construct(const FArguments& InArgs, const TSharedPtr<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow, TSharedPtr<FDisasterRecoverySessionManager> InSessionManager);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	// Toolbar buttons.
	TSharedRef<SWidget> MakeToolbarWidget();
	bool IsDeleteButtonEnabled() const;
	FReply OnDeleteClicked();
	FReply OnConfigClicked();
	FReply OnImportClicked();

	// Sessions tree view.
	TSharedRef<SWidget> MakeSessionTreeView();
	void OnSessionAdded(TSharedRef<FDisasterRecoverySession> InAddedSession);
	void OnSessionUpdated(TSharedRef<FDisasterRecoverySession> InUpdatedSession);
	void OnSessionRemoved(const FGuid& SessionRepositoryId);
	void OnSessionSelectionChanged(TSharedPtr<FDisasterRecoverySessionTreeNode> SelectedNode, ESelectInfo::Type SelectInfo);
	TSharedRef<ITableRow> OnGenerateSessionTreeNodeWidget(TSharedPtr<FDisasterRecoverySessionTreeNode> TreeNode, const TSharedRef<STableViewBase>& OwnerTable);
	void OnGetSessionTreeNodeChildren(TSharedPtr<FDisasterRecoverySessionTreeNode> InParent, TArray<TSharedPtr<FDisasterRecoverySessionTreeNode>>& OutChildren) const;
	void OnSetSessionTreeExpansionRecursive(TSharedPtr<FDisasterRecoverySessionTreeNode> InTreeNode, bool bInIsItemExpanded);

	// Selected session activities.
	TSharedRef<SWidget> MakeSessionActivityView();
	bool ShouldDisplayActivityDetails() const { return true; }
	bool FetchActivities(TArray<TSharedPtr<FConcertSessionActivity>>& InOutActivities, int32& OutFetchedCount, FText& ErrorMsg);
	FText GetNoActivityDisplayedReason() const;

	// Recover buttons.
	FText GetRecoverAllButtonTooltip() const;
	FReply OnRecoverAllButtonClicked();
	void OnRecoverThroughButtonClicked(TSharedPtr<FConcertSessionActivity> ThroughActivity);
	FReply RecoverThrough(TSharedPtr<FConcertSessionActivity> ThroughActivity);
	bool IsRecoverAllButtonEnabled() const;
	bool IsRecoverThroughButtonVisible() const;

	// Cancel/Dismiss.
	FReply OnCancelButtonClicked();
	FReply DismissWidget();

private:
	/** The session manager. */
	TSharedPtr<FDisasterRecoverySessionManager> SessionManager;

	/** The widget displaying the list of sessions. */
	TSharedPtr<STreeView<TSharedPtr<FDisasterRecoverySessionTreeNode>>> SessionTreeView;
	TSharedPtr<FDisasterRecoverySessionTreeNode> LiveCategoryRootNode;
	TSharedPtr<FDisasterRecoverySessionTreeNode> UnreviewedCrashCategoryRootNode;
	TSharedPtr<FDisasterRecoverySessionTreeNode> RecentCategoryRootNode;
	TSharedPtr<FDisasterRecoverySessionTreeNode> ImportedCategoryRootNode;
	TSharedPtr<FDisasterRecoverySessionTreeNode> SelectedSessionNode;
	TArray<TSharedPtr<FDisasterRecoverySessionTreeNode>> SessionTreeRootNodes;

	/** The widget displaying the activities of currently selected session. */
	TSharedPtr<SConcertSessionRecovery> ActivityView;
	TSharedPtr<FConcertActivityStream> SelectedSessionActivityStream;
	TSharedPtr<SWidget> NoActivityPanel;
	FText LoadingSessionErrorMsg;
	bool bLoadingActivities = false;

	TWeakPtr<SDockTab> OwnerTab;
	TWeakPtr<SWindow> OwnerWindow;
	FDateTime NextRefreshTime;
};
