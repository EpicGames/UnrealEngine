// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Logging/LogMacros.h"

class IConcertSyncClient;

DECLARE_LOG_CATEGORY_EXTERN(LogDisasterRecovery, Log, All);

/**
 * Interface for the Disaster Recovery module.
 */
class IDisasterRecoveryClientModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IDisasterRecoveryClientModule& Get()
	{
		static const FName ModuleName = "DisasterRecoveryClient";
		return FModuleManager::LoadModuleChecked<IDisasterRecoveryClientModule>(ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() during shutdown if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		static const FName ModuleName = "DisasterRecoveryClient";
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	/**
	 * Get the sync client that will performing the DisasterRecovery role
	 * @return The client
	 */
	virtual TSharedPtr<IConcertSyncClient> GetClient() const = 0;
};
